[CoreRedirects]
; AnimNextStateTree plugin -> UAFStateTree plugin
+PackageRedirects=(OldName="/Script/AnimNextStateTree", NewName="/Script/UAFStateTree")

+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeEvaluatorBase", NewName="/Script/UAFStateTree.AnimNextStateTreeEvaluatorBase")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeTaskBase", NewName="/Script/UAFStateTree.AnimNextStateTreeTaskBase")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeTraitSharedData", NewName="/Script/UAFStateTree.AnimNextStateTreeTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeRigVMConditionInstanceData", NewName="/Script/UAFStateTree.AnimNextStateTreeRigVMConditionInstanceData")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeRigVMConditionBase", NewName="/Script/UAFStateTree.AnimNextStateTreeRigVMConditionBase")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextGraphInstanceTaskInstanceData", NewName="/Script/UAFStateTree.AnimNextGraphInstanceTaskInstanceData")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeGraphInstanceTask", NewName="/Script/UAFStateTree.AnimNextStateTreeGraphInstanceTask")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeRigVMTaskInstanceData", NewName="/Script/UAFStateTree.AnimNextStateTreeRigVMTaskInstanceData")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeRigVMTaskBase", NewName="/Script/UAFStateTree.AnimNextStateTreeRigVMTaskBase")
+StructRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTreeTraitContext", NewName="/Script/UAFStateTree.AnimNextStateTreeTraitContext")

+ClassRedirects=(OldName="/Script/AnimNextStateTree.AnimNextStateTree", NewName="/Script/UAFStateTree.AnimNextStateTree")
+ClassRedirects=(OldName="/Script/AnimNextStateTree.StateTreeAnimNextSchema", NewName="/Script/UAFStateTree.StateTreeAnimNextSchema")

+StructRedirects=(OldName="/Script/AnimNextStateTreeUncookedOnly.AnimNextStateTreeOutlinerData", NewName="/Script/UAFStateTreeUncookedOnly.AnimNextStateTreeOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextStateTreeUncookedOnly.AnimNextStateTreeStateOutlinerData", NewName="/Script/UAFStateTreeUncookedOnly.AnimNextStateTreeStateOutlinerData")

+ClassRedirects=(OldName="/Script/AnimNextStateTreeUncookedOnly.AnimNextStateTreeWorkspaceAssetUserData", NewName="/Script/UAFStateTreeUncookedOnly.AnimNextStateTreeWorkspaceAssetUserData")
+ClassRedirects=(OldName="/Script/AnimNextStateTreeUncookedOnly.AnimNextStateTree_EditorData", NewName="/Script/UAFStateTreeUncookedOnly.AnimNextStateTree_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextStateTreeUncookedOnly.AnimNextStateTreeTreeEditorData", NewName="/Script/UAFStateTreeUncookedOnly.AnimNextStateTreeTreeEditorData")
+ClassRedirects=(OldName="/Script/AnimNextStateTreeUncookedOnly.AnimNextStateTreeFunctionLibraryHelper", NewName="/Script/UAFStateTreeUncookedOnly.AnimNextStateTreeFunctionLibraryHelper")

+ClassRedirects=(OldName="/Script/AnimNextStateTreeEditor.AnimNextStateTreeFactory", NewName="/Script/UAFStateTreeEditor.AnimNextStateTreeFactory")
+ClassRedirects=(OldName="/Script/AnimNextStateTreeEditor.AssetDefinition_AnimNextStateTree", NewName="/Script/UAFStateTreeEditor.AssetDefinition_AnimNextStateTree")
