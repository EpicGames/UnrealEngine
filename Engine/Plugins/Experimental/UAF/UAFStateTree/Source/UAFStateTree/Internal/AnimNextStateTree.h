// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StateTree.h"
#include "Graph/AnimNextAnimationGraph.h"

#include "AnimNextStateTree.generated.h"

namespace UE::UAF::UncookedOnly
{
struct FUtils;
}

struct UAFSTATETREE_API FAnimNextStateTreeCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Added unique name to inner state tree
		InnerStateTreeUniqueName,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	/** The GUID for this custom version number */
	const static FGuid GUID;

private:
	FAnimNextStateTreeCustomVersion() = default;
};

UCLASS(BlueprintType)
class UAFSTATETREE_API UAnimNextStateTree : public UUAFAnimGraph
{
	GENERATED_BODY()

public:
	friend class UAnimNextStateTree_EditorData;
	friend class UAnimNextStateTreeTreeEditorData;
	friend class UAnimNextStateTreeFactory;

	UPROPERTY()
	TObjectPtr<UStateTree> StateTree;

public:

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface
};