// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Compilation/AnimNextGetFunctionHeaderCompileContext.h"

class UUAFSharedVariables_EditorData;
class URigVMGraph;

struct FRigVMCompileSettings;
struct FRigVMClient;

/**
 * Helper struct to contain info needed to compile RigVM Variables
 */
struct UAFSTATETREE_API FAnimNextStateTreeProgrammaticFunctionHeaderParams
{
	FAnimNextStateTreeProgrammaticFunctionHeaderParams(const UUAFSharedVariables_EditorData* InEditorData, const FRigVMCompileSettings& InSettings, const FRigVMClient& InRigVMClient, FAnimNextGetFunctionHeaderCompileContext& InOutCompileContext)
		: EditorData(InEditorData)
		, Settings(InSettings)
		, RigVMClient(InRigVMClient)
		, OutCompileContext(InOutCompileContext)
	{
	}

	const UUAFSharedVariables_EditorData* EditorData;
	const FRigVMCompileSettings& Settings;
	const FRigVMClient& RigVMClient;
	FAnimNextGetFunctionHeaderCompileContext& OutCompileContext;
};

#endif // WITH_EDITOR