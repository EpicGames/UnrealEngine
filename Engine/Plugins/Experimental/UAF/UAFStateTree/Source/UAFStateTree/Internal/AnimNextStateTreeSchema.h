// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeSchema.h"

#include "AnimNextStateTreeSchema.generated.h"


UCLASS(HideDropDown)
class UAFSTATETREE_API UStateTreeAnimNextSchema : public UStateTreeSchema
{
	GENERATED_BODY()
public:
	UStateTreeAnimNextSchema();
	
	static const FName AnimStateTreeExecutionContextName;

	virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override;
	virtual bool IsClassAllowed(const UClass* InScriptStruct) const override;
	virtual bool IsExternalItemAllowed(const UStruct& InStruct) const override;
	
	virtual TConstArrayView<FStateTreeExternalDataDesc> GetContextDataDescs() const override { return ContextDataDescs; }
	virtual EStateTreeParameterDataType GetGlobalParameterDataType() const override;
	
#if WITH_EDITOR
	virtual bool AllowUtilityConsiderations() const override { return false; }
	virtual bool AllowEvaluators() const override { return false; }
	virtual bool AllowQueuedCompilation() const override { return false; }
#endif // WITH_EDITOR
protected:
	/** List of named external data required by schema and provided to the state tree through the execution context. */
	UPROPERTY()
	TArray<FStateTreeExternalDataDesc> ContextDataDescs;
};
