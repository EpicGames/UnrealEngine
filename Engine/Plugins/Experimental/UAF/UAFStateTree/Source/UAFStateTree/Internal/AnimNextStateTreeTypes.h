// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StateTreeEvaluatorBase.h" 
#include "StateTreeTaskBase.h"

#include "AnimNextStateTreeTypes.generated.h"

/**
 * Base struct for all AnimNext StateTree Evaluators.
 */
USTRUCT(meta = (DisplayName = "UAF Evaluator Base", Hidden))
struct UAFSTATETREE_API FAnimNextStateTreeEvaluatorBase : public FStateTreeEvaluatorBase
{
	GENERATED_BODY()
};

/**
 * Base struct for all AnimNext StateTree Tasks.
 */
USTRUCT(meta = (DisplayName = "UAF Task Base", Hidden))
struct UAFSTATETREE_API FAnimNextStateTreeTaskBase : public FStateTreeTaskBase
{
	GENERATED_BODY()

#if WITH_EDITOR
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects, const FStateTreeDataView InstanceDataView) const {} 
#endif // WITH_EDITOR
};
