// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AnimNextStateTree.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

#include "UAF/UAFAssetData.h"
#include "UAF/AnimNodeCore/UAFAnimNodeData.h"
#include "UAF/AnimNodeCore/UAFTransitionNodeData.h"
#include "UAF/AnimNodes/IUAFAnimNodeDataHasAsset.h"
#include "UAF/AnimNodes/UAFBlendStack.h"

#include "UAFStateTreeNode.generated.h"

namespace UE::UAF::StateTree
{
	
struct FUAFStateTreeNodeData;

// Anim Node for playing a State tree
class FUAFStateTreeNode : public FUAFBlendStack
{
public:
	FUAFStateTreeNode(FUAFAnimGraphUpdateContext& Context, TObjectPtr<const UStateTree> StateTree);

	UAFSTATETREE_API virtual void PreUpdate(FUAFAnimGraphUpdateContext& Context) override;
	UAFSTATETREE_API static void AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector);

	UAFSTATETREE_API void BlendTo(FUAFAnimGraphUpdateContext& Context, TConstStructView<FUAFGraphFactoryAsset> AssetData, FUAFTransitionNodeData* TransitionBlend = nullptr);

#if UAF_TRACE_ENABLED
		virtual FString GetDebugName() const override;
		virtual UStruct* GetDebugStruct() const override;
#endif

private:

	TObjectPtr<const UStateTree> StateTree = nullptr;
	FStateTreeInstanceData InstanceData;
	FStateTreeExecutionContext::FExternalGlobalParameters StateTreeExternalParameters;
};

USTRUCT(DisplayName = "State Tree")
struct FUAFStateTreeNodeData : public FUAFAnimNodeData
#if CPP
, public IUAFAnimNodeDataHasAsset
#endif
{
	GENERATED_BODY()
	
	UAFSTATETREE_API virtual FUAFAnimNodePtr CreateInstance(FUAFAnimGraphUpdateContext& Context) const override;
	UAFSTATETREE_API virtual void* GetInterface(FUAFAnimNodeInterfaceId Id) override;

	// HasAsset interface
	virtual UObject* GetAsset() const override
	{
		return StateTree;
	}

	const UStateTree* GetStateTree() const
	{
		return StateTree ? StateTree->StateTree : nullptr;
	}

	void SetStateTree(UAnimNextStateTree* InStateTree)
	{
		StateTree = InStateTree;
	}

private:

	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<UAnimNextStateTree> StateTree;
};
	
}