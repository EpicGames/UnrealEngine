// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AnimNextStateTree.h"
#include "StateTreeConditionBase.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

#include "UAF/UAFAssetData.h"
#include "UAFStateTreeContext.h"

#include "UAFStateTreeNodeConditions.generated.h"

namespace UE::UAF::StateTree
{
	
struct FUAFStateTreeNodeData;

USTRUCT()
struct FUAFDummyInstanceData
{
	GENERATED_BODY()
};

// These 2 State Tree conditions are temporary and only exist to test the state tree AnimNode implementation until variables can be better integrated

USTRUCT(MinimalAPI, Meta = (Hidden))
struct FUAFVariableConditionBase : public FStateTreeConditionBase
{
	GENERATED_BODY()

	using FInstanceDataType = FUAFDummyInstanceData;

	FUAFVariableConditionBase() = default;

	virtual bool Link(FStateTreeLinker& Linker) override
	{
		Linker.LinkExternalData(ContextHandle);
		return true;
	}

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }

	TStateTreeExternalDataHandle<FUAFStateTreeContext> ContextHandle;	
};

UENUM()
enum class EUAFFloatCompareConditionType
{
	Less,
	Greater,
};

USTRUCT(MinimalAPI, DisplayName = "(PROTOTYPE) UAF Float Variable Compare")
struct FUAFFloatCompareCondition : public FUAFVariableConditionBase
{
	GENERATED_BODY()

	FUAFFloatCompareCondition() = default;

	virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = Default)
	FAnimNextVariableReference Left;

	UPROPERTY(EditAnywhere, Category = Default)
	EUAFFloatCompareConditionType Compare = EUAFFloatCompareConditionType::Less;

	UPROPERTY(EditAnywhere, Category = Default)
	double Right = 0;
};


USTRUCT(MinimalAPI, DisplayName = "(PROTOTYPE) UAF Enum Variable Compare")
struct FUAFEnumCompareCondition : public FUAFVariableConditionBase
{
	GENERATED_BODY()

	FUAFEnumCompareCondition() = default;

	virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = Default)
	FAnimNextVariableReference Left;

	UPROPERTY(EditAnywhere, Category = Default)
	uint8 Right = 0;
};

}