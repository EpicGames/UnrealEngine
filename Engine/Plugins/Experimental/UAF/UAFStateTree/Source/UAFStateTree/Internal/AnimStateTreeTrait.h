// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/IGarbageCollection.h"
#include "TraitInterfaces/IUpdate.h"

#include "StateTree.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeReference.h"

#include "AnimStateTreeTrait.generated.h"

USTRUCT(meta = (DisplayName = "State Tree"))
struct FAnimNextStateTreeTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default", DisplayName = "State Tree", meta=(ExportAsReference="true"))
	FStateTreeReference StateTreeReference;
	
	UPROPERTY(EditAnywhere, Category = "Default")
	FStateTreeReferenceOverrides LinkedStateTreeOverrides;

	// Latent pin support boilerplate
#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \
GeneratorMacro(StateTreeReference) \
GeneratorMacro(LinkedStateTreeOverrides) \

GENERATE_TRAIT_LATENT_PROPERTIES(FAnimNextStateTreeTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
struct FStateTreeTrait : FAdditiveTrait, IUpdate, IGarbageCollection
{
	DECLARE_ANIM_TRAIT(FStateTreeTrait, FAdditiveTrait)

	using FSharedData = FAnimNextStateTreeTraitSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
		TObjectPtr<const UStateTree> StateTree;
		FStateTreeInstanceData InstanceData;
		FStateTreeExecutionContext::FExternalGlobalParameters StateTreeExternalParameters;

		void Destruct(const FExecutionContext& Context, const FTraitBinding& Binding);
	};


	// IUpdate impl
	virtual void OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;
	virtual void PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const override;

	// IGarbageCollection impl
	virtual void AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const override;
};
}