// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextFunctionHandle.h"
#include "Variables/AnimNextSharedVariables.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "StateTreeConditionBase.h"
#include "StateTreeReference.h"
#include "Traits/CallFunction.h"

#include "AnimNextStateTreeRigVMConditionBase.generated.h"

class UUAFSharedVariables;
class UUAFSharedVariables_EditorData;
class URigVMGraph;
class UStateTreeState;

struct FAnimNextStateTreeProgrammaticFunctionHeaderParams;
struct FStateTreeBindableStructDesc;
struct FStateTreeExecutionContext;
struct FUAFStateTreeContext;
struct FRigVMClient;

#define UE_API UAFSTATETREE_API

USTRUCT()
struct FAnimNextStateTreeRigVMConditionInstanceData
{
	GENERATED_BODY()

	/** Parameters to use to call the function */
	UPROPERTY(EditAnywhere, Category = "Parameters", Meta = (FixedLayout, ShowOnlyInnerProperties))
	FInstancedPropertyBag Parameters;

	/** Cached function handle */
	UE::UAF::FFunctionHandle FunctionHandle;
};

/**
 * Wrapper for RigVM based Conditions. 
 */
USTRUCT(MinimalAPI)
struct FAnimNextStateTreeRigVMConditionBase : public FStateTreeConditionBase
{
	GENERATED_BODY()

	using FInstanceDataType = FAnimNextStateTreeRigVMConditionInstanceData;

public:

	//~ Begin FStateTreeConditionBase Interface
	virtual bool Link(FStateTreeLinker& Linker) override;
	virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;
	//~ End FStateTreeConditionBase Interface

	//~ Begin FStateTreeNodeBase Interface
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual void PostEditNodeChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView) override;
#endif
	//~ End FStateTreeNodeBase Interface
	
#if WITH_EDITOR
	/** Adds condition function headers to parent StateTree RigVM */
	virtual void GetProgrammaticFunctionHeaders(FAnimNextStateTreeProgrammaticFunctionHeaderParams& InProgrammaticFunctionHeaderParams, UStateTreeState* State, FStateTreeDataView InstanceDataView, const FStateTreeBindableStructDesc& Desc);

	/** Regenerates the instance data params */
	void RegenerateInstanceDataFunctionParams(TNotNull<FInstanceDataType*> InstanceData);
#endif

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "RigVMFunctionHeader is deprecated, use CallFunctionInfo.FunctionHeader"))
	FRigVMGraphFunctionHeader RigVMFunctionHeader = FRigVMGraphFunctionHeader();

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "ConditionFunctionName is deprecated, use CallFunctionInfo.Function"))
	FName ConditionFunctionName = NAME_None;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "InternalEventName is deprecated, use CallFunctionInfo.FunctionEvent"))
	FName InternalEventName = NAME_None;

	/** Bag to hold defaults for function signature. Used to perform manual delta serialization for defaults as InstancedPropertyBag doesn't support that */
	UPROPERTY()
	FInstancedPropertyBag DefaultValues;
#endif

	/** Info needed to call a UAF function. Also has a custom function picker. */
	UPROPERTY(EditAnywhere, Category = "Call Function", meta = (Hidden))
	FUAFCallFunctionInfo CallFunctionInfo;

	/** Owning state name, populated during programmatic graph creation */
	UPROPERTY()
	FName StateName = NAME_None;

	/** External node ID defined by owning state tree, populated during programmatic graph creation */
	UPROPERTY()
	FGuid NodeId = FGuid();

	UPROPERTY()
	int32 ResultIndex = INDEX_NONE;

	UE_API void PostSerialize(const FArchive& Ar);

public:
	TStateTreeExternalDataHandle<FUAFStateTreeContext> TraitContextHandle;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FAnimNextStateTreeRigVMConditionBase> : public TStructOpsTypeTraitsBase2<FAnimNextStateTreeRigVMConditionBase>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif // WITH_EDITORONLY_DATA

#undef UE_API
