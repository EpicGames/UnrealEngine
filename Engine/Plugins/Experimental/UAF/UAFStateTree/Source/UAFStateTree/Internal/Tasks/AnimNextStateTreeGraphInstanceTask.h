// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextStateTreeTypes.h"
#include "Graph/AnimNextAnimationGraph.h"
#include <limits>

#include "AlphaBlend.h"
#include "Factory/AnimNextFactoryParams.h"
#include "StructUtils/PropertyBag.h"
#include "UAF/UAFAssetData.h"

#include "AnimNextStateTreeGraphInstanceTask.generated.h"

struct FUAFStateTreeContext;
class UUAFBlendProfile;

USTRUCT()
struct UAFSTATETREE_API FAnimNextGraphInstanceTaskInstanceData
{
	GENERATED_BODY()

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAnimNextGraphInstanceTaskInstanceData() = default;
	~FAnimNextGraphInstanceTaskInstanceData() = default;
	FAnimNextGraphInstanceTaskInstanceData(const FAnimNextGraphInstanceTaskInstanceData&) = default;
	FAnimNextGraphInstanceTaskInstanceData(FAnimNextGraphInstanceTaskInstanceData&&) = default;
	FAnimNextGraphInstanceTaskInstanceData& operator=(const FAnimNextGraphInstanceTaskInstanceData&) = default;
	FAnimNextGraphInstanceTaskInstanceData& operator=(FAnimNextGraphInstanceTaskInstanceData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// This needs to be a set of 'active' instanced structs/property bags.
	// Available structs/property bags are populated based around asset key (query factory in editor)
	// Checkbox per 'available' struct in the UI only
	// Hitting the checkbox adds the instanced struct/property bag to the defaults
	// Checkbox-struct still visible if factory gets available struct removed (no data loss!)
	// Upgrade on load all structs/property bags (bags need source asset to reference)

#if WITH_EDITORONLY_DATA
	// The asset to instantiate
	UE_DEPRECATED(5.8, "Use AssetData instead")
	UPROPERTY()
	TObjectPtr<UObject> Asset_DEPRECATED = nullptr;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = Animation)
	TInstancedStruct<FUAFGraphFactoryAsset> AssetData; 

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FInstancedPropertyBag Payload_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	// Factory params for procedural graphs
	UE_DEPRECATED(5.8, "Use AssetData instead")
	UPROPERTY()
	FAnimNextFactoryParams Parameters_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	// Blend options for when the state is pushed
	UPROPERTY(EditAnywhere, Category = Animation)
	FAlphaBlendArgs BlendOptions;

	// Optional blend profile to use when the state is pushed
	UPROPERTY(EditAnywhere, Category = Animation)
	TObjectPtr<UUAFBlendProfile> BlendProfile;

	// Whether this task should continue to tick once state is entered
	UPROPERTY(EditAnywhere, Category = Animation)
	bool bContinueTicking = true;

	// How early to trigger complete. Setting this allows for a blend out while the timeline is still playing.
	UPROPERTY(EditAnywhere, Category = Animation)
	float CompleteBlendOutTime = 0.0f;

	// Current playback ratio 
	UPROPERTY(VisibleAnywhere, Category = Animation)
	float PlaybackRatio = 1.0f;

	// Current time remaining
	UPROPERTY(VisibleAnywhere, Category = Animation)
	float TimeLeft = std::numeric_limits<float>::infinity();

	// Current graph duration
	UPROPERTY(VisibleAnywhere, Category = Animation)
	float Duration = 0.0f;

	// Current looping status
	UPROPERTY(VisibleAnywhere, Category = Animation)
	bool bIsLooping = false;

#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
#endif
};

template<>
struct TStructOpsTypeTraits<FAnimNextGraphInstanceTaskInstanceData> : public TStructOpsTypeTraitsBase2<FAnimNextGraphInstanceTaskInstanceData>
{
#if WITH_EDITORONLY_DATA
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
#endif
};


// Basic task pushing AnimationGraph onto blend stack
USTRUCT(meta = (DisplayName = "UAF Graph"))
struct UAFSTATETREE_API FAnimNextStateTreeGraphInstanceTask : public FAnimNextStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FAnimNextGraphInstanceTaskInstanceData;

	FAnimNextStateTreeGraphInstanceTask();
	
	virtual bool Link(FStateTreeLinker& Linker) override;
protected:
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects, const FStateTreeDataView InstanceDataView) const override;
#endif
public:
	TStateTreeExternalDataHandle<FUAFStateTreeContext> TraitContextHandle;
};
