// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextFunctionHandle.h"
#include "AnimNextStateTreeTypes.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "StateTreeTaskBase.h"
#include "Traits/CallFunction.h"

#include "AnimNextStateTreeRigVMTaskBase.generated.h"

class UAnimNextDataInterface;
class UAnimNextDataInterface_EditorData;
class URigVMGraph;
class UStateTreeState;

struct FAnimNextStateTreeProgrammaticFunctionHeaderParams;
struct FStateTreeBindableStructDesc;
struct FStateTreeExecutionContext;
struct FRigVMClient;
struct FUAFStateTreeContext;

#define UE_API UAFSTATETREE_API

UENUM()
enum class EStateTreeFunctionCallFrequency
{
	OnEnter,
	OnTick,
	OnExit,
};

USTRUCT()
struct UE_API FAnimNextStateTreeRigVMTaskInstanceData
{
	GENERATED_BODY()

	/** Parameters to use to call the function */
	UPROPERTY(EditAnywhere, Category = "Parameters", Meta = (FixedLayout, ShowOnlyInnerProperties))
	FInstancedPropertyBag Parameters;

	/** Cached function handle */
	UE::UAF::FFunctionHandle FunctionHandle;
};

/**
 * Wrapper for RigVM based Tasks. 
 */
USTRUCT(MinimalAPI)
struct FAnimNextStateTreeRigVMTaskBase : public FAnimNextStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FAnimNextStateTreeRigVMTaskInstanceData;

public:

	//~ Begin FStateTreeTaskBase Interface
	virtual bool Link(FStateTreeLinker& Linker) override;
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	//~ End FStateTreeTaskBase Interface

	//~ Begin FStateTreeNodeBase Interface
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text) const override;
	virtual void PostEditNodeChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView) override;
#endif
	//~ End FStateTreeNodeBase Interface
	
#if WITH_EDITOR
	/** Adds Task function headers to parent StateTree RigVM */
	virtual void GetProgrammaticFunctionHeaders(FAnimNextStateTreeProgrammaticFunctionHeaderParams& InProgrammaticFunctionHeaderParams, UStateTreeState* State, FStateTreeDataView InstanceDataView, const FStateTreeBindableStructDesc& Desc);

	/** Regenerates the instance data params */
	void RegenerateInstanceDataFunctionParams(TNotNull<FInstanceDataType*> InstanceData);
#endif

public:

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "RigVMFunctionHeader is deprecated, use CallFunctionInfo.FunctionHeader"))
	FRigVMGraphFunctionHeader RigVMFunctionHeader = FRigVMGraphFunctionHeader();
	
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "TaskFunctionName is deprecated, use CallFunctionInfo.Function"))
	FName TaskFunctionName = NAME_None;
	
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "InternalEventName is deprecated, use CallFunctionInfo.FunctionEvent"))
	FName InternalEventName = NAME_None;

	/** Bag to hold defaults for function signature. Used to perform manual delta serialization for defaults as InstancedPropertyBag doesn't support that */
	UPROPERTY()
	FInstancedPropertyBag DefaultValues;
#endif

	/** Info needed to call a UAF function. Also has a custom function picker. */
	UPROPERTY(EditAnywhere, Category = "Call Function", meta = (Hidden))
	FUAFCallFunctionInfo CallFunctionInfo;

	UPROPERTY(EditAnywhere, Category = "Call Function")
	EStateTreeFunctionCallFrequency CallFunctionFrequency = EStateTreeFunctionCallFrequency::OnEnter;

	/** Owning state name, populated during programmatic graph creation */
	UPROPERTY()
	FName StateName = NAME_None;

	/** External node ID defined by owning state tree, populated during programmatic graph creation */
	UPROPERTY()
	FGuid NodeId = FGuid();

	UPROPERTY()
	int32 ResultIndex = INDEX_NONE;

public:

	UE_API void PostSerialize(const FArchive& Ar);

public:

	TStateTreeExternalDataHandle<FUAFStateTreeContext> TraitContextHandle;

protected:

	/** Internal impl to call some RigVM function */
	EStateTreeRunStatus CallFunction(FStateTreeExecutionContext& Context) const;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FAnimNextStateTreeRigVMTaskBase> : public TStructOpsTypeTraitsBase2<FAnimNextStateTreeRigVMTaskBase>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif // WITH_EDITORONLY_DATA

#undef UE_API