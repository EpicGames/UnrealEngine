// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextStateTreeTypes.h"
#include "StateTreeLinker.h"
#include "StructUtils/PropertyBag.h"
#include "UAFStateTreeContext.h"
#include "Variables/AnimNextVariableReference.h"

#include "UAFStateTreeSetVariableTask.generated.h"

struct FStateTreeExecutionContext;
struct FStateTreeTransitionResult;

/** Instance data for FUAFStateTreeSetVariableTask. */
USTRUCT()
struct FUAFStateTreeSetVariableTaskInstanceData
{
	GENERATED_BODY()

	/** Which UAF variable to write. */
	UPROPERTY(EditAnywhere, Category = Variable)
	FAnimNextVariableReference Variable;

	/** Value to write. Automatically reshaped to match Variable's type when Variable changes. */
	UPROPERTY(EditAnywhere, Category = Variable, Meta = (FixedLayout, ShowOnlyInnerProperties))
	FInstancedPropertyBag Value;
};

/**
 * StateTree task that writes a single UAF shared variable on state entry.
 * Stack multiple instances of this task in one state to write multiple variables independently.
 */
USTRUCT(MinimalAPI, DisplayName = "UAF Set Variable")
struct FUAFStateTreeSetVariableTask : public FAnimNextStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FUAFStateTreeSetVariableTaskInstanceData;

	//~ Begin FStateTreeTaskBase Interface
	virtual bool Link(FStateTreeLinker& Linker) override;
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	//~ End FStateTreeTaskBase Interface

	//~ Begin FStateTreeNodeBase Interface
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
#if WITH_EDITOR
	virtual void PostEditInstanceDataChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView) override;
#endif
	//~ End FStateTreeNodeBase Interface

	TStateTreeExternalDataHandle<FUAFStateTreeContext> ContextHandle;
};
