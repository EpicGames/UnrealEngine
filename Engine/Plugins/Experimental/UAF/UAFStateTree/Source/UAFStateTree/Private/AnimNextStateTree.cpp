// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTree.h"

#include "Serialization/CustomVersion.h"

const FGuid FAnimNextStateTreeCustomVersion::GUID(0x45641511, 0x102F42BB, 0xA6EF181D, 0x6C442CAC);
namespace UE::UAF::StateTree::Private
{
FCustomVersionRegistration GRegisterStateTreeCustomVersion(FAnimNextStateTreeCustomVersion::GUID, FAnimNextStateTreeCustomVersion::LatestVersion, TEXT("AnimNextStateTree"));
}

void UAnimNextStateTree::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	const int32 CurrentVersion = GetLinkerCustomVersion(FAnimNextStateTreeCustomVersion::GUID);
	if (CurrentVersion < FAnimNextStateTreeCustomVersion::InnerStateTreeUniqueName)
	{
		if (StateTree && StateTree->GetFName() == "StateTree")
		{
			StateTree->Rename(*(GetFName().ToString() + "_StateTree"));
		}
	}
#endif // WITH_EDITORONLY_DATA
}