// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeModule.h"

#include "AnimNextStateTree.h"
#include "StateTree.h"
#include "StateTreeReference.h"
#include "AnimNode/UAFStateTreeNode.h"
#include "Graph/AnimNextAnimationGraph.h"
#include "RigVMCore/RigVMRegistry.h"
#include "UAF/AnimNodeCore/UAFAnimNodeFactory.h"

namespace UE::UAF::StateTree
{

void FAnimNextStateTreeModule::StartupModule()
{
	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UStateTree::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		{ UUAFAnimGraph::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class }
	};

	FRigVMRegistry::Get().RegisterObjectTypes(AllowedObjectTypes);

	static UScriptStruct* const AllowedStructTypes[] =
	{
		FStateTreeReference::StaticStruct(),
		FStateTreeReferenceOverrides::StaticStruct(),
	};

	FRigVMRegistry::Get().RegisterStructTypes(AllowedStructTypes);

	
	UAFStateTreeClassPath = FUAFAnimNodeFactory::RegisterAsset(UAnimNextStateTree::StaticClass(), [](UObject* Object, FUAFAnimGraphUpdateContext& Context)
		{
			UAnimNextStateTree* StateTree = CastChecked<UAnimNextStateTree>(Object);
			return MakeAnimNode<FUAFStateTreeNode>(Context, StateTree->StateTree);
		});
}
	
void FAnimNextStateTreeModule::ShutdownModule()
{
	FUAFAnimNodeFactory::UnregisterAsset(UAFStateTreeClassPath);
}

IMPLEMENT_MODULE(FAnimNextStateTreeModule, UAFStateTree)

}
