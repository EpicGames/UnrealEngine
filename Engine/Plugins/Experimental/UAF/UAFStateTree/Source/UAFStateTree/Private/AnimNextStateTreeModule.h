// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAnimNextStateTreeModule.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::UAF::StateTree
{

class FAnimNextStateTreeModule : public IAnimNextStateTreeModule
{
public:
	virtual void ShutdownModule() override;
	virtual void StartupModule() override;
	
private:
	FTopLevelAssetPath UAFStateTreeClassPath;
};

}
