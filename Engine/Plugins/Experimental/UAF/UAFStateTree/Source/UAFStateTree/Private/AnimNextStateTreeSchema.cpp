// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeSchema.h"

#include "AnimNextStateTreeTypes.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreePropertyFunctionBase.h"
#include "StateTreeTaskBase.h"
#include "Components/ActorComponent.h"
#include "Subsystems/WorldSubsystem.h"
#include "GameFramework/Actor.h"
#include "UAFStateTreeContext.h"
#include "AnimNode/UAFStateTreeNodeConditions.h"
#include "Conditions/AnimNextStateTreeRigVMConditionBase.h"

const FName UStateTreeAnimNextSchema::AnimStateTreeExecutionContextName("ExecutionContext");

UStateTreeAnimNextSchema::UStateTreeAnimNextSchema() : ContextDataDescs(
	{{ AnimStateTreeExecutionContextName, FUAFStateTreeContext::StaticStruct(), FGuid(0xDFB93B7F, 0xEDBE4906, 0x851C66B2, 0x7585FA21) }})
{
}

bool UStateTreeAnimNextSchema::IsStructAllowed(const UScriptStruct* InScriptStruct) const
{
	return InScriptStruct->IsChildOf(FStateTreeConditionCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreeEvaluatorCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreeTaskCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreeConsiderationCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FAnimNextStateTreeRigVMConditionBase::StaticStruct())
	|| InScriptStruct->IsChildOf(UE::UAF::StateTree::FUAFVariableConditionBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FStateTreePropertyFunctionCommonBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FAnimNextStateTreeEvaluatorBase::StaticStruct())
	|| InScriptStruct->IsChildOf(FAnimNextStateTreeTaskBase::StaticStruct());
}

bool UStateTreeAnimNextSchema::IsClassAllowed(const UClass* InClass) const
{
	return false;
}

bool UStateTreeAnimNextSchema::IsExternalItemAllowed(const UStruct& InStruct) const
{
	return InStruct.IsChildOf<FUAFStateTreeContext>();
}

EStateTreeParameterDataType UStateTreeAnimNextSchema::GetGlobalParameterDataType() const
{
	return EStateTreeParameterDataType::ExternalGlobalParameterData;
}