// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode/UAFStateTreeNode.h"

#include "AnimNextStateTreeSchema.h"
#include "StateTreeExecutionContext.h"
#include "UAFStateTreeNodeContext.h"
#include "Script/UAFRigVMComponent.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "UAF/AnimNodeCore/UAFAnimNodeUpdate.h"
#include "UAF/AnimNodes/IUAFAnimNodeTimeline.h"
#include "UAF/AnimNodes/UAFSimpleTransition.h"
#include "UAF/AnimNodeCore/UAFGraphFactoryAssetAnimNodeFactory.h"
#include "ObjectTrace.h"

#if ENABLE_ANIM_DEBUG 
#include "Debugger/StateTreeRuntimeValidation.h"
#endif // ENABLE_ANIM_DEBUG

namespace UE::UAF::StateTree
{
	
FUAFStateTreeNode::FUAFStateTreeNode(FUAFAnimGraphUpdateContext& Context, TObjectPtr<const UStateTree> InStateTree)
	: FUAFBlendStack(Context)
{
	InitializeAs<FUAFStateTreeNode>(Context);
	
	StateTree = InStateTree;

	if (StateTree)
	{
		UObject* Owner = GetTransientPackage();
#if ENABLE_ANIM_DEBUG 
		Owner = const_cast<UObject*>(Context.GetHostObject());
		// @TODO: Makes unique, but breaks visual logger
		Owner = Owner ? Owner->GetOuter() : Owner;

		if (!Owner)
		{
			return;
		}
#endif // ENABLE_ANIM_DEBUG

		FStateTreeExecutionContext StateTreeExecutionContext(*Owner, *StateTree, InstanceData);
	
		FUAFStateTreeNodeContext UAFStateTreeContext(Context, this);
		StateTreeExecutionContext.SetContextDataByName(UStateTreeAnimNextSchema::AnimStateTreeExecutionContextName, FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), &UAFStateTreeContext));

#if UAF_TRACE_ENABLED
		UAF_TRACE_ANIMNODE(Context, this);
		StateTreeExecutionContext.SetOuterTraceId(DebugInstanceId);
#endif

		const FInstancedPropertyBag& StateTreeParameters = StateTree->GetDefaultParameters();
		StateTreeExternalParameters.Reset();
	
		TArray<int32> StartPropertyOffsets;
		FUAFRigVMComponent& RigVMComponent = Context.GetVariablesOwner()->GetOrAddComponent<FUAFRigVMComponent>();
		const FRigVMExtendedExecuteContext& ExtendedExecuteContext = RigVMComponent.GetExtendedExecuteContext();

		// Variable Binding:  It might be possible to get this block of code working in a way where it binds the state tree variables to their values in the System/Module, but as-is the layouts mismatch and it fails.
		// #if ENABLE_ANIM_DEBUG 
		// if (!Context.GetVariablesOwner()->LayoutMatches(StateTree->GetDefaultParameters()))
		// {
		// 	UE_LOGF(LogAnimation, Error, "Anim StateTree Parameter Layout Mismatch: %ls - StateTree: %ls", *Owner->GetFName().ToString(),  *StateTree->GetOuter()->GetFName().ToString());
		// 	StateTreeExecutionContext.Stop(EStateTreeRunStatus::Failed);
		// 	return;
		//}
		// #endif // ENABLE_ANIM_DEBUG 
		//
		// const int32 NumVariables = StateTreeParameters.GetNumPropertiesInBag();
		// for(int32 VariableIndex = 0; VariableIndex < NumVariables; ++VariableIndex)
		// {
		// 	const FPropertyBagPropertyDesc& Desc = StateTreeParameters.GetPropertyBagStruct()->GetPropertyDescs()[VariableIndex];
		// 	StartPropertyOffsets.Add(Desc.CachedProperty->GetOffset_ForInternal());
		// }
		//
		// for (const FPropertyBindingCopyInfoBatch& Batch : StateTree->GetPropertyBindings().GetCopyBatches())
		// {
		// 	for (const FPropertyBindingCopyInfo& Copy : StateTree->GetPropertyBindings().Super::GetBatchCopies(Batch))
		// 	{
		// 		const FStateTreeDataHandle& Handle = Copy.SourceDataHandle.Get<FStateTreeDataHandle>();
		// 		if (Handle.GetSource() == EStateTreeDataSourceType::ExternalGlobalParameterData)
		// 		{
		//
		// 			const int32 RequiredOffset = [&Copy]()
		// 			{
		// 				if(Copy.SourceIndirection.Type == EPropertyBindingPropertyAccessType::Offset)
		// 				{
		// 					return (int32)Copy.SourceIndirection.Offset;
		// 				}
		//
		// 				checkf(false, TEXT("Only expecting offset indirections for remapping"));
		// 				return Copy.SourceLeafProperty->GetOffset_ForInternal();
		// 			}();
		//
		// 			uint8* MemoryPtr = nullptr;
		// 			const int32 NumOffsets = StartPropertyOffsets.Num();
		// 			for (int32 Index = 0; Index < NumOffsets; ++Index)
		// 			{
		// 				const int32 NextOffset = (Index < (NumOffsets - 1)) ? StartPropertyOffsets[Index + 1] : INDEX_NONE;
		// 				if (RequiredOffset >= StartPropertyOffsets[Index] && (RequiredOffset < NextOffset || NextOffset == INDEX_NONE))
		// 				{
		// 					// Incorporate the UPropertyBag root-level property offset into the remapped memory-ptr itself, so that the property-access indirection works "as-normal"
		// 					MemoryPtr = (uint8*)ExtendedExecuteContext.ExternalVariableRuntimeData[Index].Memory - StartPropertyOffsets[Index];
		// 					break;
		// 				}
		// 			}
		// 			check(MemoryPtr != nullptr);
		//
		// 			if (StateTreeExternalParameters.Add(Copy, MemoryPtr))
		// 			{
		// #if ENABLE_ANIM_DEBUG 
		// 						UE_CLOGF(CVarLogPropertyBindingMemoryPtrInfo.GetValueOnAnyThread(), LogAnimation, Warning, "Mapped: Source: %ls\nTarget: %lsSize: %i\nOffset: %i\nType: %ls", *Copy.SourceLeafProperty->GetName(), *Copy.TargetLeafProperty->GetName(), Copy.CopySize, RequiredOffset, *FindObject<UEnum>(nullptr, TEXT("/Script/PropertyBindingUtils.EPropertyCopyType"))->GetNameStringByValue(static_cast<int64>(Copy.Type)));
		// 					}
		// 					else
		// 					{
		// 						UE_CLOGF(CVarLogPropertyBindingMemoryPtrInfo.GetValueOnAnyThread(), LogAnimation, Warning, "Skipped: Source: %ls\nTarget: %lsSize: %i\nOffset: %i\nType: %ls", *Copy.SourceLeafProperty->GetName(), *Copy.TargetLeafProperty->GetName(), Copy.CopySize, RequiredOffset, *FindObject<UEnum>(nullptr, TEXT("/Script/PropertyBindingUtils.EPropertyCopyType"))->GetNameStringByValue(static_cast<int64>(Copy.Type)));
		// #endif // ENABLE_ANIM_DEBUG 
		// 			}
		// 		}
		// 	}
		// }
	
		// StateTreeExecutionContext.SetExternalGlobalParameters(&StateTreeExternalParameters);

		StateTreeExecutionContext.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateLambda([this, &UAFStateTreeContext](const FStateTreeExecutionContext& Context, const UStateTree* InStateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews) -> bool
			{
				for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
				{
					const FStateTreeExternalDataDesc& ItemDesc = ExternalDataDescs[Index];
					if (ItemDesc.Struct != nullptr)
					{
						if (ItemDesc.Struct->IsChildOf(FUAFStateTreeContext::StaticStruct()))
						{
							OutDataViews[Index] = FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), reinterpret_cast<uint8*>(&UAFStateTreeContext));
						}
					}
				}

				return true;
			}));

		if (StateTreeExecutionContext.IsValid())
		{
			StateTreeExecutionContext.Start(); // todo: call Stop in destructor?
		}
	}
}

void FUAFStateTreeNode::PreUpdate(FUAFAnimGraphUpdateContext& Context)
{
	SCOPED_NAMED_EVENT(Controller_Update_StateTree, FColor::Blue);

	FUAFBlendStack::PreUpdate(Context);

	if (StateTree)
	{
		UObject* Owner = GetTransientPackage();
#if ENABLE_ANIM_DEBUG 
		Owner = const_cast<UObject*>(Context.GetHostObject());
		// @TODO: Makes unique, but breaks visual logger
		Owner = Owner ? Owner->GetOuter() : Owner;

		if (!Owner)
		{
			Owner = GetTransientPackage();
		}
#endif // ENABLE_ANIM_DEBUG 
	
		FStateTreeExecutionContext StateTreeExecutionContext(*Owner, *StateTree, InstanceData);
	
		if (StateTreeExecutionContext.GetLastTickStatus() != EStateTreeRunStatus::Failed)
		{
			FUAFStateTreeNodeContext UAFStateTreeContext(Context, this);
			StateTreeExecutionContext.SetContextDataByName(UStateTreeAnimNextSchema::AnimStateTreeExecutionContextName, FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), reinterpret_cast<uint8*>(&UAFStateTreeContext)));
			StateTreeExecutionContext.SetExternalGlobalParameters(&StateTreeExternalParameters);
#if OBJECT_TRACE_ENABLED
			StateTreeExecutionContext.SetOuterTraceId(FObjectTrace::GetObjectId(Owner)); // todo setting outer id to outer object for now
#endif
			StateTreeExecutionContext.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateLambda([this, &UAFStateTreeContext]( const FStateTreeExecutionContext& Context, const UStateTree* InStateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews) -> bool
			{
				for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
				{
					const FStateTreeExternalDataDesc& ItemDesc = ExternalDataDescs[Index];
					if (ItemDesc.Struct != nullptr)
					{
					   if (ItemDesc.Struct->IsChildOf(FUAFStateTreeContext::StaticStruct()))
					   {
						   OutDataViews[Index] = FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), reinterpret_cast<uint8*>(&UAFStateTreeContext));
					   }
				   }
			   }
		
			   return true;
		   }));

			StateTreeExecutionContext.Tick(Context.GetDeltaTime());
		}
	}

	ValidateSingleChild();
}
	
void FUAFStateTreeNode::AddReferencedObjects(FUAFAnimNode* This, FReferenceCollector& Collector)
{
	FUAFStateTreeNode* That = static_cast<FUAFStateTreeNode*>(This);
	Collector.AddReferencedObject(That->StateTree);
}

void FUAFStateTreeNode::BlendTo(FUAFAnimGraphUpdateContext& Context, TConstStructView<FUAFGraphFactoryAsset> AssetData, FUAFTransitionNodeData* TransitionBlend)
{
	FUAFAnimNodePtr TargetInstance = FUAFGraphFactoryAssetAnimNodeFactory::CreateUAFAnimNodeFromObject(AssetData, Context);
	TransitionTo(Context, TargetInstance,  TransitionBlend);
}

#if UAF_TRACE_ENABLED
FString FUAFStateTreeNode::GetDebugName() const
{
	if (StateTree)
	{
		return StateTree->GetName();
	}
	else
	{
		return "State Tree";
	}
}

UStruct* FUAFStateTreeNode::GetDebugStruct() const
{
	return FUAFStateTreeNodeData::StaticStruct();
}
#endif

FUAFAnimNodePtr  FUAFStateTreeNodeData::CreateInstance(FUAFAnimGraphUpdateContext& Context) const
{
	if (StateTree && StateTree->StateTree)
	{
		return MakeAnimNode<FUAFStateTreeNode>(Context, StateTree->StateTree);
	}
	else
	{
		return nullptr;
	}
}

void* FUAFStateTreeNodeData::GetInterface(FUAFAnimNodeInterfaceId Id)
{
	if (Id == IUAFAnimNodeDataHasAsset::InterfaceId)
	{
		return static_cast<IUAFAnimNodeDataHasAsset*>(this);
	}

	return nullptr;
}
	
}