// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode/UAFStateTreeNodeConditions.h"

#include "AnimNextStateTreeSchema.h"
#include "StateTreeExecutionContext.h"
#include "UAFStateTreeNodeContext.h"
#include "Script/UAFRigVMComponent.h"
#include "Logging/LogScopedVerbosityOverride.h"

#if ENABLE_ANIM_DEBUG 
#include "Debugger/StateTreeRuntimeValidation.h"
#endif // ENABLE_ANIM_DEBUG

namespace UE::UAF::StateTree
{
	
bool FUAFFloatCompareCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	FUAFStateTreeContext& ExecContext = Context.GetExternalData(ContextHandle);

	if (FUAFAssetInstance* VariablesOwner = ExecContext.GetVariablesOwner())
	{
		double LeftValue;
		if (VariablesOwner->GetVariable(Left, LeftValue) == EPropertyBagResult::Success)
		{	
			if (Compare == EUAFFloatCompareConditionType::Less)
			{
				return LeftValue < Right;
			}
			return LeftValue > Right;
		}
	}
	return false;
}

bool FUAFEnumCompareCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	FUAFStateTreeContext& ExecContext = Context.GetExternalData(ContextHandle);

	if (FUAFAssetInstance* VariablesOwner = ExecContext.GetVariablesOwner())
	{
		uint8 LeftValue;
		if (VariablesOwner->GetVariable(Left, LeftValue) == EPropertyBagResult::Success)
		{
			return LeftValue == Right;
		}
	}

	return false;
}

}