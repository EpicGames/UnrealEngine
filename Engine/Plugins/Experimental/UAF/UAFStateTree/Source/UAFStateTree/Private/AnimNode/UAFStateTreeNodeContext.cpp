// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFStateTreeNodeContext.h"
#include "AnimNode/UAFStateTreeNode.h"
#include "UAF/AnimNodes/IUAFAnimNodeTimeline.h"

namespace UE::UAF::StateTree
{

void FUAFStateTreeNodeContext::QueryPlaybackInfo(FPlaybackInfo& OutPlaybackInfo) const
{
	if (AnimNode)
	{
		// todo: make templated GetInterface calls work correctly when called on a specific node type.
		if (const UE::UAF::IUAFAnimNodeTimeline* Timeline = AnimNode->FUAFAnimNode::GetInterface<IUAFAnimNodeTimeline>())
		{
			float Duration = Timeline->GetLength();
			float CurrentTime = Timeline->GetCurrentTime();
			OutPlaybackInfo.PlaybackRatio = Duration > 0.f ? CurrentTime / Duration : 1.0f;
			OutPlaybackInfo.Duration = Duration;
			// todo: handle negative playback rates (need something exposed from ITimeline)
			OutPlaybackInfo.TimeLeft = Duration - CurrentTime;
			OutPlaybackInfo.bIsLooping = Timeline->IsLooping();
		}
	}
}

bool FUAFStateTreeNodeContext::PushAssetOntoBlendStack(FGraphAssetHandleConstView InAsset, const FAlphaBlendArgs& InBlendArguments, const UUAFBlendProfile* BlendProfile) const
{
	using namespace UE::UAF;

	if (Context && AnimNode)
	{
		if (InAsset.IsValid())
		{
			AnimNode->BlendTo(*Context, InAsset);
			// todo: use InBlendArguments to create an appropriate Transition
			return true;
		}
	}
	return false;
}

FUAFAssetInstance* FUAFStateTreeNodeContext::GetVariablesOwner() const
{
	if (Context)
	{
		return Context->GetVariablesOwner();
	}
	return nullptr;
}

}