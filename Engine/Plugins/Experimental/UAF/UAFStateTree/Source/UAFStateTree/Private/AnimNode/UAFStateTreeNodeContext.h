// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFStateTreeContext.h"
#include "UAFStateTreeNodeContext.generated.h"

namespace UE::UAF
{
class FUAFAnimGraphUpdateContext;
}

class UUAFBlendProfile;

namespace UE::UAF::StateTree
{
class FUAFStateTreeNode;

USTRUCT()
struct FUAFStateTreeNodeContext : public FUAFStateTreeContext
{	
	GENERATED_BODY()
	
	FUAFStateTreeNodeContext() 
		: Context(nullptr), AnimNode(nullptr)
	{
	}

	FUAFStateTreeNodeContext(FUAFAnimGraphUpdateContext& InContext, FUAFStateTreeNode* InAnimNode)
		: Context(&InContext), AnimNode(InAnimNode)
	{
	}

	virtual bool PushAssetOntoBlendStack(FGraphAssetHandleConstView InAsset, const FAlphaBlendArgs& InBlendArguments, const UUAFBlendProfile* BlendProfile) const override;
    virtual void QueryPlaybackInfo(FPlaybackInfo& OutPlaybackInfo) const override;
	virtual FUAFAssetInstance* GetVariablesOwner() const override;

	UE::UAF::FUAFAnimGraphUpdateContext* GetContext() const
	{
		return Context;
	}

	FUAFStateTreeNode* GetAnimNode() const
	{
		return AnimNode;
	}

private:

	FUAFAnimGraphUpdateContext* Context;
	FUAFStateTreeNode* AnimNode;
};

}
