// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimStateTreeTrait.h"

#include "UAFStateTreeTraitContext.h"
#include "AnimNextStateTreeSchema.h"
#include "StateTreeExecutionContext.h"
#include "Script/UAFRigVMComponent.h"
#include "Graph/AnimNextGraphInstance.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "TraitCore/NodeInstance.h"

#if ENABLE_ANIM_DEBUG 
#include "Debugger/StateTreeRuntimeValidation.h"
#endif // ENABLE_ANIM_DEBUG 

namespace UE::UAF
{
	AUTO_REGISTER_ANIM_TRAIT(FStateTreeTrait)

	// Trait implementation boilerplate
	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
	GeneratorMacro(IUpdate) \
	GeneratorMacro(IGarbageCollection) \

	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FStateTreeTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

#if ENABLE_ANIM_DEBUG 
	TAutoConsoleVariable<bool> CVarLogPropertyBindingMemoryPtrInfo(TEXT("a.StateTree.LogPropertyBindingMemoryPtrInfo"), false, TEXT("Log information while generating property binding memory pointer mappings between RigVM and StateTree"));
#endif //ENABLE_ANIM_DEBUG 
	
	void FStateTreeTrait::FInstanceData::Destruct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
#if WITH_STATETREE_DEBUG
		// @TODO: UE-240683
		// We should call stop, but at the moment we don't have a good time / way to do that 
		/*if (StateTree)
		{
			UObject* Owner = GetTransientPackage();
			
			FStateTreeExecutionContext StateTreeExecutionContext(*Owner, *StateTree, InstanceData);
			FAnimNextGraphInstance* OwnerGraphInstance = &Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
			if(OwnerGraphInstance && StateTreeExecutionContext.GetLastTickStatus() != EStateTreeRunStatus::Failed)
			{
				FUAFStateTreeTraitContext TraitContext(*const_cast<FExecutionContext*>(&Context), &Binding);
				StateTreeExecutionContext.SetContextDataByName(UStateTreeAnimNextSchema::AnimStateTreeExecutionContextName, FStateTreeDataView(FUAFStateTreeTraitContext::StaticStruct(), reinterpret_cast<uint8*>(&TraitContext)));
				StateTreeExecutionContext.SetExternalGlobalParameters(&StateTreeExternalParameters);

				StateTreeExecutionContext.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateLambda([this, &TraitContext]( const FStateTreeExecutionContext& Context, const UStateTree* InStateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews) -> bool
				{
					for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
					{
						const FStateTreeExternalDataDesc& ItemDesc = ExternalDataDescs[Index];
						if (ItemDesc.Struct != nullptr)
						{
							if (ItemDesc.Struct->IsChildOf(FUAFStateTreeTraitContext::StaticStruct()))
							{
								OutDataViews[Index] = FStateTreeDataView(FUAFStateTreeTraitContext::StaticStruct(), reinterpret_cast<uint8*>(&TraitContext));
							}
						}
					}
		
					return true;
				}));

				StateTreeExecutionContext.Stop();
			}			
		}*/	
#endif
		FTrait::FInstanceData::Destruct(Context, Binding);
	}

	void FStateTreeTrait::OnBecomeRelevant(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		InstanceData->StateTree = SharedData->GetStateTreeReference(Binding).GetStateTree();

		if (InstanceData->StateTree)
		{
			UObject* Owner = GetTransientPackage();
#if ENABLE_ANIM_DEBUG 
			Owner = const_cast<UObject*>(Context.GetHostObject());
			// @TODO: Makes unique, but breaks visual logger
			Owner = Owner ? Owner->GetOuter() : Owner;

			if (!Owner)
			{
				return;
			}
#endif // ENABLE_ANIM_DEBUG

			FStateTreeExecutionContext StateTreeExecutionContext(*Owner, *InstanceData->StateTree, InstanceData->InstanceData);

			FUAFStateTreeTraitContext TraitContext(Context, Binding);
			StateTreeExecutionContext.SetContextDataByName(UStateTreeAnimNextSchema::AnimStateTreeExecutionContextName, FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), &TraitContext));

			FAnimNextGraphInstance& OwnerGraphInstance = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
			StateTreeExecutionContext.SetOuterTraceId(OwnerGraphInstance.GetUniqueId());

#if ENABLE_ANIM_DEBUG 
			if (!InstanceData->StateTree->IsReadyToRun())
			{
				UE_LOGF(LogAnimation, Error, "StateTree cannot be ran, may have failed compilation StateTree: %ls", *InstanceData->StateTree->GetOuter()->GetFName().ToString());
				StateTreeExecutionContext.Stop(EStateTreeRunStatus::Failed);
				return;
			}

			if (!OwnerGraphInstance.LayoutMatches(InstanceData->StateTree->GetDefaultParameters()))
			{
				UE_LOGF(LogAnimation, Error, "Anim StateTree Parameter Layout Mismatch: %ls - StateTree: %ls", *Owner->GetFName().ToString(), *InstanceData->StateTree->GetOuter()->GetFName().ToString());
				StateTreeExecutionContext.Stop(EStateTreeRunStatus::Failed);
				return;
			}
#endif // ENABLE_ANIM_DEBUG 

			const FInstancedPropertyBag& StateTreeParameters = InstanceData->StateTree->GetDefaultParameters();
			InstanceData->StateTreeExternalParameters.Reset();
			
			TArray<int32> StartPropertyOffsets;
			FUAFRigVMComponent& RigVMComponent = OwnerGraphInstance.GetOrAddComponent<FUAFRigVMComponent>();
			const FRigVMExtendedExecuteContext& ExtendedExecuteContext = RigVMComponent.GetExtendedExecuteContext();

			const int32 NumVariables = StateTreeParameters.GetNumPropertiesInBag();
			for(int32 VariableIndex = 0; VariableIndex < NumVariables; ++VariableIndex)
			{
				check(ExtendedExecuteContext.ExternalVariableRuntimeData.IsValidIndex(VariableIndex));
				const FPropertyBagPropertyDesc& Desc = StateTreeParameters.GetPropertyBagStruct()->GetPropertyDescs()[VariableIndex];
				StartPropertyOffsets.Add(Desc.CachedProperty->GetOffset_ForInternal());
			}

			for (const FPropertyBindingCopyInfoBatch& Batch : InstanceData->StateTree->GetPropertyBindings().GetCopyBatches())
			{
				for (const FPropertyBindingCopyInfo& Copy : InstanceData->StateTree->GetPropertyBindings().Super::GetBatchCopies(Batch))
				{
					const FStateTreeDataHandle& Handle = Copy.SourceDataHandle.Get<FStateTreeDataHandle>();
					if (Handle.GetSource() == EStateTreeDataSourceType::ExternalGlobalParameterData)
					{

						const int32 RequiredOffset = [&Copy]()
						{
							if(Copy.SourceIndirection.Type == EPropertyBindingPropertyAccessType::Offset)
							{
								return (int32)Copy.SourceIndirection.Offset;
							}

							checkf(false, TEXT("Only expecting offset indirections for remapping"));
							return Copy.SourceLeafProperty->GetOffset_ForInternal();
						}();

						uint8* MemoryPtr = nullptr;
						const int32 NumOffsets = StartPropertyOffsets.Num();
						for (int32 Index = 0; Index < NumOffsets; ++Index)
						{
							const int32 NextOffset = (Index < (NumOffsets - 1)) ? StartPropertyOffsets[Index + 1] : INDEX_NONE;
							if (RequiredOffset >= StartPropertyOffsets[Index] && (RequiredOffset < NextOffset || NextOffset == INDEX_NONE))
							{
								// Incorporate the UPropertyBag root-level property offset into the remapped memory-ptr itself, so that the property-access indirection works "as-normal"
								MemoryPtr = (uint8*)ExtendedExecuteContext.ExternalVariableRuntimeData[Index].Memory - StartPropertyOffsets[Index];
								break;
							}
						}
						check(MemoryPtr != nullptr);

						if (InstanceData->StateTreeExternalParameters.Add(Copy, MemoryPtr))
						{
#if ENABLE_ANIM_DEBUG 
							UE_CLOGF(CVarLogPropertyBindingMemoryPtrInfo.GetValueOnAnyThread(), LogAnimation, Warning, "Mapped: Source: %ls\nTarget: %lsSize: %i\nOffset: %i\nType: %ls", *Copy.SourceLeafProperty->GetName(), *Copy.TargetLeafProperty->GetName(), Copy.CopySize, RequiredOffset, *FindObject<UEnum>(nullptr, TEXT("/Script/PropertyBindingUtils.EPropertyCopyType"))->GetNameStringByValue(static_cast<int64>(Copy.Type)));
						}
						else
						{
							UE_CLOGF(CVarLogPropertyBindingMemoryPtrInfo.GetValueOnAnyThread(), LogAnimation, Warning, "Skipped: Source: %ls\nTarget: %lsSize: %i\nOffset: %i\nType: %ls", *Copy.SourceLeafProperty->GetName(), *Copy.TargetLeafProperty->GetName(), Copy.CopySize, RequiredOffset, *FindObject<UEnum>(nullptr, TEXT("/Script/PropertyBindingUtils.EPropertyCopyType"))->GetNameStringByValue(static_cast<int64>(Copy.Type)));
#endif // ENABLE_ANIM_DEBUG 
						}
					}
				}
			}
			StateTreeExecutionContext.SetExternalGlobalParameters(&InstanceData->StateTreeExternalParameters);
			
			StateTreeExecutionContext.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateLambda([this, &TraitContext]( const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews) -> bool
			{
				for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
				{
					const FStateTreeExternalDataDesc& ItemDesc = ExternalDataDescs[Index];
					if (ItemDesc.Struct != nullptr)
					{
						if (ItemDesc.Struct->IsChildOf(FUAFStateTreeContext::StaticStruct()))
						{
							OutDataViews[Index] = FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), reinterpret_cast<uint8*>(&TraitContext));
						}
					}
				}

				return true;
			}));

			if (StateTreeExecutionContext.IsValid())
			{
				StateTreeExecutionContext.Start();
			}
		}
	}

	void FStateTreeTrait::PreUpdate(FUpdateTraversalContext& Context, const TTraitBinding<IUpdate>& Binding, const FTraitUpdateState& TraitState) const
	{
		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		if (InstanceData->StateTree && InstanceData->StateTree->IsReadyToRun())
		{
			UObject* Owner = GetTransientPackage();
#if ENABLE_ANIM_DEBUG 
			Owner = const_cast<UObject*>(Context.GetHostObject());
			// @TODO: Makes unique, but breaks visual logger
			Owner = Owner ? Owner->GetOuter() : Owner;

			if (!Owner)
			{
				return;
			}
#endif // ENABLE_ANIM_DEBUG 
			
			FStateTreeExecutionContext StateTreeExecutionContext(*Owner, *InstanceData->StateTree, InstanceData->InstanceData);
			FAnimNextGraphInstance& OwnerGraphInstance = Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
			if (StateTreeExecutionContext.GetLastTickStatus() != EStateTreeRunStatus::Failed)
			{
				FUAFStateTreeTraitContext TraitContext(Context, Binding);
				StateTreeExecutionContext.SetContextDataByName(UStateTreeAnimNextSchema::AnimStateTreeExecutionContextName, FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), reinterpret_cast<uint8*>(&TraitContext)));
				StateTreeExecutionContext.SetExternalGlobalParameters(&InstanceData->StateTreeExternalParameters);
				StateTreeExecutionContext.SetOuterTraceId(OwnerGraphInstance.GetUniqueId());
				StateTreeExecutionContext.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateLambda([this, &TraitContext]( const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews) -> bool
				{
					for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
					{
						const FStateTreeExternalDataDesc& ItemDesc = ExternalDataDescs[Index];
						if (ItemDesc.Struct != nullptr)
						{
							if (ItemDesc.Struct->IsChildOf(FUAFStateTreeContext::StaticStruct()))
							{
								OutDataViews[Index] = FStateTreeDataView(FUAFStateTreeContext::StaticStruct(), reinterpret_cast<uint8*>(&TraitContext));
							}
						}
					}

					return true;
				}));

				StateTreeExecutionContext.Tick(TraitState.GetDeltaTime());
			}
		}

		// Update the traits below us, we updated first so we can push transitions
		IUpdate::PreUpdate(Context, Binding, TraitState);
	}

	void FStateTreeTrait::AddReferencedObjects(const FExecutionContext& Context, const TTraitBinding<IGarbageCollection>& Binding, FReferenceCollector& Collector) const
	{
		IGarbageCollection::AddReferencedObjects(Context, Binding, Collector);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		Collector.AddReferencedObject(InstanceData->StateTree);
	}
}
