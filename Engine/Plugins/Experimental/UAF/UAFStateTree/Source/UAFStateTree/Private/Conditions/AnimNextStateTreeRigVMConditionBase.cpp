// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/AnimNextStateTreeRigVMConditionBase.h"
#include "AnimNextExecuteContext.h"
#include "Graph/AnimNextGraphContextData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "StateTreeLinker.h"
#include "StateTreeExecutionContext.h"
#include "StructUtils/PropertyBag.h"
#include "TraitCore/ExecutionContext.h"
#include "UAFStateTreeContext.h"
#include "Script/UAFRigVMComponent.h"

#if WITH_EDITOR
#include "AnimNextStateTreeEditorOnlyTypes.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "UAFStateTreeRigVMUtils.h"
#include "UncookedOnlyUtils.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AnimNextStateTreeRigVMConditionBase"


#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextStateTreeRigVMConditionBase)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextStateTreeRigVMConditionBase

bool FAnimNextStateTreeRigVMConditionBase::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(TraitContextHandle);
	return true;
}

bool FAnimNextStateTreeRigVMConditionBase::TestCondition(FStateTreeExecutionContext& Context) const
{
	if (FAnimNextStateTreeRigVMConditionInstanceData* InstanceData = Context.GetInstanceDataPtr<FAnimNextStateTreeRigVMConditionInstanceData>(*this))
	{
		FUAFStateTreeContext& ExecContext = Context.GetExternalData(TraitContextHandle);
		FUAFAssetInstance* GraphInstance = ExecContext.GetVariablesOwner();
		if (!GraphInstance)
		{
			return false;
		}

		// Need const cast as VM execution isn't const.
		if (UUAFRigVMAsset* RigVMInstance = const_cast<UUAFRigVMAsset*>(GraphInstance->GetAsset<UUAFRigVMAsset>()))
		{
#if WITH_EDITOR
			if (!InstanceData->FunctionHandle.IsValidForVM(RigVMInstance->GetRigVM()->GetVMHash()))
#else
			if (!InstanceData->FunctionHandle.IsValid())
#endif
			{
				InstanceData->FunctionHandle = RigVMInstance->GetFunctionHandle(CallFunctionInfo.FunctionEvent);
			}

			if (!ensure(InstanceData->FunctionHandle.IsValid() && ResultIndex != INDEX_NONE))
			{
				return false;
			}

			FUAFRigVMComponent& RigVMComponent = GraphInstance->GetOrAddComponent<FUAFRigVMComponent>();
			FRigVMExtendedExecuteContext& ExtendedExecuteContext = RigVMComponent.GetExtendedExecuteContext();
			FAnimNextExecuteContext& AnimNextContext = ExtendedExecuteContext.GetPublicDataSafe<FAnimNextExecuteContext>();

			FUAFAssetContextData ContextData(*GraphInstance, CallFunctionInfo.FunctionEvent, AnimNextContext.GetDeltaTime<float>());
			UE::UAF::FScopedExecuteContextData ContextDataScope(AnimNextContext, ContextData);

			RigVMInstance->CallFunctionHandle(InstanceData->FunctionHandle, ExtendedExecuteContext, InstanceData->Parameters);

			// Pull the result out of the property bag
			return InstanceData->Parameters.GetValueBool(InstanceData->Parameters.GetPropertyBagStruct()->GetPropertyDescs()[ResultIndex]).GetValue();
		}
	}

	return true;
}

#if WITH_EDITOR
FText FAnimNextStateTreeRigVMConditionBase::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	if (CallFunctionInfo.Function.IsValid())
	{
		return FText::Format(LOCTEXT("UAF_StateTreeConditon_Desc", "Function ({0})"), FText::FromName(CallFunctionInfo.Function));
	}
	return LOCTEXT("UAF_StateTreeConditon_FunctionMissingDesc", "Function (NONE)");
}

void FAnimNextStateTreeRigVMConditionBase::PostEditNodeChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView)
{
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNextStateTreeRigVMConditionBase, CallFunctionInfo))
	{
		// Function Name selected has changed. Update param struct & result / event names used during execution.
		if (FAnimNextStateTreeRigVMConditionInstanceData* InstanceData = InstanceDataView.GetMutablePtr<FAnimNextStateTreeRigVMConditionInstanceData>())
		{
			RegenerateInstanceDataFunctionParams(InstanceData);
		}
	}
}

void FAnimNextStateTreeRigVMConditionBase::GetProgrammaticFunctionHeaders(FAnimNextStateTreeProgrammaticFunctionHeaderParams& InProgrammaticFunctionHeaderParams, UStateTreeState* State, FStateTreeDataView InstanceDataView, const FStateTreeBindableStructDesc& Desc)
{
	using namespace UE::UAF::UncookedOnly;

	FAnimNextGetFunctionHeaderCompileContext& OutCompileContext = InProgrammaticFunctionHeaderParams.OutCompileContext;

	StateName = State->Name;
	NodeId = Desc.ID;

	// Validate that our cached instance parameters match the latest RigVM signature. Update if needed.
	UStateTree* OwningStateTree = State->GetTypedOuter<UStateTree>();
	if (FAnimNextStateTreeRigVMConditionInstanceData* InstanceData = InstanceDataView.GetMutablePtr<FAnimNextStateTreeRigVMConditionInstanceData>())
	{
		if (UE::UAF::StateTree::FUtils::IsBindingOutDated(CallFunctionInfo, DefaultValues, InstanceData->Parameters))
		{
			RegenerateInstanceDataFunctionParams(InstanceData);
		}
	}

	FAnimNextProgrammaticFunctionHeader AnimNextFunctionHeader = {};
	AnimNextFunctionHeader.FunctionHeader = CallFunctionInfo.FunctionHeader;
	OutCompileContext.AddUniqueFunctionHeader(AnimNextFunctionHeader);
}

void FAnimNextStateTreeRigVMConditionBase::RegenerateInstanceDataFunctionParams(TNotNull<FInstanceDataType*> InstanceData)
{
	UE::UAF::StateTree::FUtils::RegenerateParameterPropertyBag(CallFunctionInfo, DefaultValues, InstanceData->Parameters, ResultIndex);
}

#endif // WITH_EDITOR

void FAnimNextStateTreeRigVMConditionBase::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading())
	{
		if (RigVMFunctionHeader.IsValid() && !ConditionFunctionName.IsNone() && !InternalEventName.IsNone())
		{
			CallFunctionInfo.FunctionHeader = RigVMFunctionHeader;
			CallFunctionInfo.Function = ConditionFunctionName;
			CallFunctionInfo.FunctionEvent = InternalEventName;

			RigVMFunctionHeader = {};
			ConditionFunctionName = {};
			InternalEventName = {};
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

#undef LOCTEXT_NAMESPACE
