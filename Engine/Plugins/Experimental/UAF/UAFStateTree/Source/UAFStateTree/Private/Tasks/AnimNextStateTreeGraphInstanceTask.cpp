// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AnimNextStateTreeGraphInstanceTask.h"

#include "StateTreeLinker.h"
#include "StateTreeExecutionContext.h"
#include "UAFStateTreeContext.h"
#include "Asset/UAFAnimGraphAssetData.h"
#include "Factory/AnimGraphFactory.h"
#include "Traits/BlendSpacePlayerTraitData.h"
#include "Traits/SequencePlayerTraitData.h"
#include "UAF/UAFAssetFactory.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UAF/AnimNodeCore/UAFAnimNode.h"
#include "UAF/AnimNodes/IUAFAnimNodeTimeline.h"

#define LOCTEXT_NAMESPACE "AnimNextStateTreeGraphInstanceTask"

#if WITH_EDITORONLY_DATA
bool FAnimNextGraphInstanceTaskInstanceData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	return false;
}

void FAnimNextGraphInstanceTaskInstanceData::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() == false)
	{
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
	
	// We've had multiple deprecations here so we need to do things in stages
	// History is we went from Payload -> Parameters + Object* -> UAF Asset
	
	// First, create a UAF asset from a deprecated object ptr
	// We must do this first since its the only way to get the default parameters
	if (CustomVersion < FFortniteMainBranchObjectVersion::UAFAssetData)
	{
		// Prior to this we only supported 'imprecise' payloads so try to do our best be existing content, but some data loss is possible
		if (Asset_DEPRECATED != nullptr && AssetData.IsValid() == false)
		{
			AssetData = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFGraphFactoryAsset>(Asset_DEPRECATED);
		}
	}

	// now handle conversion of payload -> parameters
	if (CustomVersion < FFortniteMainBranchObjectVersion::AnimNextVariableReferences)
	{
		// Prior to this we only supported 'imprecise' payloads so try to do our best be existing content, but some data loss is possible
		if (AssetData.IsValid() && !AssetData.GetPtr<FUAFGraphFactoryAsset_Graph>())
		{
			Parameters_DEPRECATED = UE::UAF::FAnimGraphFactory::GetDefaultParamsForAsset(AssetData);

			// Match any legacy parameter values by name
			if (Payload_DEPRECATED.GetNumPropertiesInBag() > 0 && Parameters_DEPRECATED.Builder.Stacks.Num() > 0)
			{
				const UPropertyBag* PropertyBag = Payload_DEPRECATED.GetPropertyBagStruct();
				const uint8* SourceContainerPtr = Payload_DEPRECATED.GetValue().GetMemory();

				auto PatchInLegacyPropertyByName = [this, SourceContainerPtr](const FPropertyBagPropertyDesc& InDesc)
					{
						for (FAnimNextSimpleAnimGraphBuilderTraitDesc& TraitDesc : Parameters_DEPRECATED.Builder.Stacks[0].TraitDescs)
						{
							uint8* TargetContainerPtr = TraitDesc.TraitData.GetMutableMemory();
							const UScriptStruct* ScriptStruct = TraitDesc.TraitData.GetScriptStruct();
							if (TargetContainerPtr && ScriptStruct)
							{
								for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
								{
									auto NameMatches = [](const FProperty* InProperty, FName InNameA, FName InNameB)
									{
										if (InNameA == InNameB)
										{
											return true;
										}

										// Match bool properties with no 'b' prefix
										if (InProperty->IsA<FBoolProperty>())
										{
											FString StringA = InNameA.ToString();
											StringA.RemoveFromStart(TEXT("b"));
											FString StringB = InNameB.ToString();
											StringB.RemoveFromStart(TEXT("b"));
											return StringA.Equals(StringB, ESearchCase::IgnoreCase);
										}
										return false;
									};

									if (NameMatches(*It, It->GetFName(), InDesc.Name) && It->GetClass() == InDesc.CachedProperty->GetClass())
									{
										const uint8* SourcePtr = InDesc.CachedProperty->ContainerPtrToValuePtr<uint8>(SourceContainerPtr);
										uint8* TargetPtr = It->ContainerPtrToValuePtr<uint8>(TargetContainerPtr);
										It->CopyCompleteValue(TargetPtr, SourcePtr);
										break;
									}
								}
							}
						}
					};

				for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
				{
					PatchInLegacyPropertyByName(Desc);
				}
			}
		}
	}

	// Now handle conversion of parameters -> UAF asset
	if (CustomVersion < FFortniteMainBranchObjectVersion::UAFAssetData)
	{
		if (AssetData.IsValid())
		{
			// Do some manual conversion of parameters to asset data
			// only check first entry in stack
			if (Parameters_DEPRECATED.Builder.Stacks.Num() > 0)
			{
				auto FindTraitByType = []<typename TTraitData>(const TArray<FAnimNextSimpleAnimGraphBuilderTraitDesc>& TraitDescs)
				{
					const FAnimNextSimpleAnimGraphBuilderTraitDesc* FoundItem = TraitDescs.FindByPredicate( [](const FAnimNextSimpleAnimGraphBuilderTraitDesc& Candidate)
					{
						return Candidate.TraitData.IsValid() && Candidate.TraitData.GetScriptStruct() == TTraitData::StaticStruct(); 
					});

					return FoundItem != nullptr ? FoundItem->TraitData.GetPtr<TTraitData>() : nullptr;
				};
				
				if (FUAFGraphFactoryAsset_Animation* AnimationAsset = AssetData.GetMutablePtr<FUAFGraphFactoryAsset_Animation>())
				{
					if (const UE::UAF::FSequencePlayerData* SequenceTraitData = FindTraitByType.operator()<UE::UAF::FSequencePlayerData>(Parameters_DEPRECATED.Builder.Stacks[0].TraitDescs))
					{
						if (ensure(AnimationAsset->AnimationSequence == SequenceTraitData->AnimSequence)) // We expect this to have been equal in the first place
						{
							AnimationAsset->LoopMode = SequenceTraitData->LoopMode;
							AnimationAsset->PlayRate = SequenceTraitData->PlayRate;
						}
					}
				}
				else if (FUAFGraphFactoryAsset_BlendSpace* BlendSpaceAsset = AssetData.GetMutablePtr<FUAFGraphFactoryAsset_BlendSpace>())
				{
					if (const UE::UAF::FBlendSpacePlayerData* BlendSpaceData = FindTraitByType.operator()<UE::UAF::FBlendSpacePlayerData>(Parameters_DEPRECATED.Builder.Stacks[0].TraitDescs))
					{
						if (ensure(BlendSpaceAsset->BlendSpace == BlendSpaceData->BlendSpace)) // We expect this to have been equal in the first place
						{
							BlendSpaceAsset->XAxisSamplePoint = BlendSpaceData->XAxisSamplePoint;
							BlendSpaceAsset->YAxisSamplePoint = BlendSpaceData->YAxisSamplePoint;
							BlendSpaceAsset->PlayRate = BlendSpaceData->PlayRate;
						}
					}
				}
			}
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

FAnimNextStateTreeGraphInstanceTask::FAnimNextStateTreeGraphInstanceTask()
{
	// Re-selecting the same state should not cause a re-trigger of EnterState()
	bShouldStateChangeOnReselect = false;
}

bool FAnimNextStateTreeGraphInstanceTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(TraitContextHandle);	
	return true;
}

EStateTreeRunStatus FAnimNextStateTreeGraphInstanceTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FUAFStateTreeContext& ExecContext = Context.GetExternalData(TraitContextHandle);
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	if (ExecContext.PushAssetOntoBlendStack(InstanceData.AssetData, InstanceData.BlendOptions, InstanceData.BlendProfile))
	{
		return EStateTreeRunStatus::Running;
	}
	
	return EStateTreeRunStatus::Failed;
}

EStateTreeRunStatus FAnimNextStateTreeGraphInstanceTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	FUAFStateTreeContext& ExecContext = Context.GetExternalData(TraitContextHandle);

	if (InstanceData.bContinueTicking)
	{
		FUAFStateTreeContext::FPlaybackInfo PlaybackInfo;
		ExecContext.QueryPlaybackInfo(PlaybackInfo);
		
		InstanceData.PlaybackRatio = PlaybackInfo.PlaybackRatio;
		InstanceData.Duration = PlaybackInfo.Duration;
		InstanceData.TimeLeft = PlaybackInfo.TimeLeft;
		InstanceData.bIsLooping = PlaybackInfo.bIsLooping;

		// We can get a fallback timeline with duration of 0.0f during graph init / compile.
		// Avoid finishing the task for this edge case
		// @TODO: Later on consider making 0.0f duration invalid in timeline state.
		if (InstanceData.Duration != 0.0f)
		{
			if (InstanceData.TimeLeft - InstanceData.CompleteBlendOutTime <= 0.0f && !InstanceData.bIsLooping)
			{
				Context.FinishTask(*this, EStateTreeFinishTaskType::Succeeded);
			}
		}

		return EStateTreeRunStatus::Running;
	}
	
	return EStateTreeRunStatus::Succeeded;
}

void FAnimNextStateTreeGraphInstanceTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FAnimNextStateTreeTaskBase::ExitState(Context, Transition);
}

#if WITH_EDITOR

FText FAnimNextStateTreeGraphInstanceTask::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	if (InstanceDataView.IsValid())
	{
		const UE::UAF::FAssetHandle& AssetHandle = InstanceDataView.Get<FAnimNextGraphInstanceTaskInstanceData>().AssetData;
		if (AssetHandle.IsValid())
		{
			return FText::Format(LOCTEXT("UAF_Task_Desc", "UAF Task ({0})"), FText::FromString(AssetHandle.Get().GetName()));
		}
	}
	return LOCTEXT("UAF_Task_AssetMissingDesc", "UAF Task (NONE)");
}

void FAnimNextStateTreeGraphInstanceTask::GetObjectReferences(TArray<const UObject*>& OutReferencedObjects, FStateTreeDataView InstanceDataView) const
{
	if (InstanceDataView.IsValid())
	{
		const UE::UAF::FAssetHandle& AssetHandle = InstanceDataView.Get<FAnimNextGraphInstanceTaskInstanceData>().AssetData;
		if (AssetHandle.IsValid())
		{
			AssetHandle.Get().GetObjectReferences(OutReferencedObjects);
		}
	}
}
#endif

#undef LOCTEXT_NAMESPACE