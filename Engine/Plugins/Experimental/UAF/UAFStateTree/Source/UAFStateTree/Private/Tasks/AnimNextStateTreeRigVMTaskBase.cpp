// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AnimNextStateTreeRigVMTaskBase.h"
#include "AnimNextExecuteContext.h"
#include "Graph/AnimNextGraphContextData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "StateTreeLinker.h"
#include "StateTreeExecutionContext.h"
#include "StructUtils/PropertyBag.h"
#include "TraitCore/ExecutionContext.h"
#include "UAFStateTreeContext.h"
#include "AnimNextRigVMAsset.h"
#include "Script/UAFRigVMComponent.h"

#if WITH_EDITOR
#include "AnimNextStateTreeEditorOnlyTypes.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "UAFStateTreeRigVMUtils.h"
#include "UncookedOnlyUtils.h"
#endif // WITH_EDITOR

#define LOCTEXT_NAMESPACE "AnimNextStateTreeRigVMTaskBase"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextStateTreeRigVMTaskBase)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextStateTreeRigVMTaskBase

bool FAnimNextStateTreeRigVMTaskBase::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(TraitContextHandle);
	return true;
}

EStateTreeRunStatus FAnimNextStateTreeRigVMTaskBase::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	if (CallFunctionFrequency != EStateTreeFunctionCallFrequency::OnEnter)
	{
		return Super::EnterState(Context, Transition);
	}

	return CallFunction(Context);
}

void FAnimNextStateTreeRigVMTaskBase::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	if (CallFunctionFrequency != EStateTreeFunctionCallFrequency::OnExit)
	{
		return Super::ExitState(Context, Transition);
	}

	CallFunction(Context);
}

EStateTreeRunStatus FAnimNextStateTreeRigVMTaskBase::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	if (CallFunctionFrequency != EStateTreeFunctionCallFrequency::OnTick)
	{
		return Super::Tick(Context, DeltaTime);
	}

	return CallFunction(Context);
}

EStateTreeRunStatus FAnimNextStateTreeRigVMTaskBase::CallFunction(FStateTreeExecutionContext& Context) const
{
	if (FAnimNextStateTreeRigVMTaskInstanceData* InstanceData = Context.GetInstanceDataPtr<FAnimNextStateTreeRigVMTaskInstanceData>(*this))
	{
		FUAFStateTreeContext& ExecContext = Context.GetExternalData(TraitContextHandle);
		FUAFAssetInstance* GraphInstance = ExecContext.GetVariablesOwner();
		if (!GraphInstance)
		{
			return EStateTreeRunStatus::Failed;
		}

		// Need const cast as VM execution isn't const.
		if (UUAFRigVMAsset* RigVMAsset = const_cast<UUAFRigVMAsset*>(GraphInstance->GetAsset<UUAFRigVMAsset>()))
		{
#if WITH_EDITOR
			if (!InstanceData->FunctionHandle.IsValidForVM(RigVMAsset->GetRigVM()->GetVMHash()))
#else
			if (!InstanceData->FunctionHandle.IsValid())
#endif
			{
				InstanceData->FunctionHandle = RigVMAsset->GetFunctionHandle(CallFunctionInfo.FunctionEvent);
			}

			// See below, for now ignore index as we fire and forget on tasks
			if (!ensure(InstanceData->FunctionHandle.IsValid() /*&& ResultIndex != INDEX_NONE*/))
			{
				return EStateTreeRunStatus::Failed;
			}

			FUAFRigVMComponent& RigVMComponent = GraphInstance->GetOrAddComponent<FUAFRigVMComponent>();
			FRigVMExtendedExecuteContext& ExtendedExecuteContext = RigVMComponent.GetExtendedExecuteContext();
			FAnimNextExecuteContext& AnimNextContext = ExtendedExecuteContext.GetPublicDataSafe<FAnimNextExecuteContext>();

			FUAFAssetContextData ContextData(*GraphInstance, CallFunctionInfo.FunctionEvent, AnimNextContext.GetDeltaTime<float>());
			UE::UAF::FScopedExecuteContextData ContextDataScope(AnimNextContext, ContextData);

			RigVMAsset->CallFunctionHandle(InstanceData->FunctionHandle, ExtendedExecuteContext, InstanceData->Parameters);

			// @TODO: For now we use a fire & forget model. But in the future we might want users to be able to bind to the result value from a function
			// This will require a special UI customization, & standalone execute is useful for modifying existing vars. So save for later effort.	
			//return InstanceData->Parameters.GetValueBool(InstanceData->Parameters.GetPropertyBagStruct()->GetPropertyDescs()[ResultIndex]).GetValue();

			return EStateTreeRunStatus::Running;
		}
	}

	return EStateTreeRunStatus::Failed;
}

#if WITH_EDITOR
FText FAnimNextStateTreeRigVMTaskBase::GetDescription(const FGuid& ID, FStateTreeDataView InstanceDataView, const IStateTreeBindingLookup& BindingLookup, EStateTreeNodeFormatting Formatting) const
{
	if (CallFunctionInfo.Function.IsValid())
	{
		return FText::Format(LOCTEXT("UAF_StateTreeTask_Desc", "Function ({0})"), FText::FromName(CallFunctionInfo.Function));
	}
	return LOCTEXT("UAF_StateTreeTask_FunctionMissingDesc", "Function (NONE)");
}

void FAnimNextStateTreeRigVMTaskBase::PostEditNodeChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView)
{
	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FAnimNextStateTreeRigVMTaskBase, CallFunctionInfo))
	{
		// Function selected has changed. Update param struct & result index.
		if (FAnimNextStateTreeRigVMTaskInstanceData* InstanceData = InstanceDataView.GetMutablePtr<FAnimNextStateTreeRigVMTaskInstanceData>())
		{
			RegenerateInstanceDataFunctionParams(InstanceData);
		}
	}
}

void FAnimNextStateTreeRigVMTaskBase::GetProgrammaticFunctionHeaders(FAnimNextStateTreeProgrammaticFunctionHeaderParams& InProgrammaticFunctionHeaderParams, UStateTreeState* State, FStateTreeDataView InstanceDataView, const FStateTreeBindableStructDesc& Desc)
{
	using namespace UE::UAF::UncookedOnly;

	FAnimNextGetFunctionHeaderCompileContext& OutCompileContext = InProgrammaticFunctionHeaderParams.OutCompileContext;
	
	StateName = State->Name;
	NodeId = Desc.ID;

	// Validate that our cached instance parameters match the latest RigVM signature. Update if needed.
	UStateTree* OwningStateTree = State->GetTypedOuter<UStateTree>();
	if (FAnimNextStateTreeRigVMTaskInstanceData* InstanceData = InstanceDataView.GetMutablePtr<FAnimNextStateTreeRigVMTaskInstanceData>())
	{
		if (UE::UAF::StateTree::FUtils::IsBindingOutDated(CallFunctionInfo, DefaultValues, InstanceData->Parameters))
		{
			RegenerateInstanceDataFunctionParams(InstanceData);
		}
	}

	FAnimNextProgrammaticFunctionHeader AnimNextFunctionHeader = {};
	AnimNextFunctionHeader.FunctionHeader = CallFunctionInfo.FunctionHeader;
	OutCompileContext.AddUniqueFunctionHeader(AnimNextFunctionHeader);
}

void FAnimNextStateTreeRigVMTaskBase::RegenerateInstanceDataFunctionParams(TNotNull<FInstanceDataType*> InstanceData)
{
	UE::UAF::StateTree::FUtils::RegenerateParameterPropertyBag(CallFunctionInfo, DefaultValues, InstanceData->Parameters, ResultIndex);
}

#endif // WITH_EDITOR

void FAnimNextStateTreeRigVMTaskBase::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading())
	{
		if (RigVMFunctionHeader.IsValid() && !TaskFunctionName.IsNone() && !InternalEventName.IsNone())
		{
			CallFunctionInfo.FunctionHeader = RigVMFunctionHeader;
			CallFunctionInfo.Function = TaskFunctionName;
			CallFunctionInfo.FunctionEvent = InternalEventName;

			RigVMFunctionHeader = {};
			TaskFunctionName = {};
			InternalEventName = {};
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

#undef LOCTEXT_NAMESPACE
