// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/UAFStateTreeSetVariableTask.h"

#include "Param/ParamType.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "UAFAssetInstance.h"
#include "UAFStateTreeContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFStateTreeSetVariableTask)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FUAFStateTreeSetVariableTask

bool FUAFStateTreeSetVariableTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(ContextHandle);
	return true;
}

EStateTreeRunStatus FUAFStateTreeSetVariableTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData<FInstanceDataType>(*this);

	if (InstanceData.Variable.IsNone())
	{
		ensureMsgf(false, TEXT("UAF Set Variable: No Variable assigned. StateTree: '%s', State: '%s', Owner: '%s'"),
			*Context.GetStateTree()->GetName(),
			*Context.GetActiveStateName(),
			*Context.GetOwner()->GetName());
		return EStateTreeRunStatus::Failed;
	}

	if (!InstanceData.Value.IsValid() || InstanceData.Value.GetNumPropertiesInBag() != 1)
	{
		ensureMsgf(false, TEXT("UAF Set Variable: Value invalid '%s'. StateTree: '%s', State: '%s', Owner: '%s'"),
			*InstanceData.Variable.GetName().ToString(),
			*Context.GetStateTree()->GetName(),
			*Context.GetActiveStateName(),
			*Context.GetOwner()->GetName());
		return EStateTreeRunStatus::Failed;
	}

	FUAFStateTreeContext& ExecContext = Context.GetExternalData(ContextHandle);
	if (FUAFAssetInstance * VariablesOwner = ExecContext.GetVariablesOwner())
	{
		const UPropertyBag* BagStruct = InstanceData.Value.GetPropertyBagStruct();
		TConstArrayView<FPropertyBagPropertyDesc> Descs = BagStruct->GetPropertyDescs();
		const FPropertyBagPropertyDesc& Desc = Descs[0];

		const uint8* BagMemory = InstanceData.Value.GetValue().GetMemory();
		const uint8* PropMemory = Desc.CachedProperty->ContainerPtrToValuePtr<uint8>(BagMemory);

		const FAnimNextParamType VarType = FAnimNextParamType::FromPropertyBagPropertyDesc(Desc);

		if (EPropertyBagResult::Success == VariablesOwner->SetVariable(InstanceData.Variable, VarType, TConstArrayView<uint8>(PropMemory, Desc.CachedProperty->GetSize())))
		{
			return EStateTreeRunStatus::Running;
		}
	}

	return EStateTreeRunStatus::Failed;
}

#if WITH_EDITOR
void FUAFStateTreeSetVariableTask::PostEditInstanceDataChangeChainProperty(const FPropertyChangedChainEvent& PropertyChangedEvent, FStateTreeDataView InstanceDataView)
{
	FInstanceDataType* InstanceData = InstanceDataView.GetMutablePtr<FInstanceDataType>();
	if (!InstanceData)
	{
		return;
	}

	// Only reshape the Value bag when Variable itself has changed
	if (PropertyChangedEvent.GetMemberPropertyName() != GET_MEMBER_NAME_CHECKED(FUAFStateTreeSetVariableTaskInstanceData, Variable))
	{
		return;
	}

	// Resolve the FProperty for the selected variable to determine its type
	const FProperty* VarProp = InstanceData->Variable.ResolveProperty();
	if (!VarProp)
	{
		InstanceData->Value.Reset();
		return;
	}

	// Reshape Value to a single property matching the variable's type, preserving any existing value
	FPropertyBagPropertyDesc NewDesc(InstanceData->Variable.GetName(), VarProp);
	FInstancedPropertyBag TempBag;
	TempBag.AddProperties({ NewDesc });
	InstanceData->Value.MigrateToNewBagInstance(TempBag);
}
#endif // WITH_EDITOR
