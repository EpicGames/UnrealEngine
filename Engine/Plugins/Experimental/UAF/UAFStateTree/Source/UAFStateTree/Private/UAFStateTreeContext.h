// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAF/UAFAssetData.h"

#include "UAFStateTreeContext.generated.h"

struct FAlphaBlendArgs;
struct FUAFAssetInstance;
class UUAFBlendProfile;

USTRUCT()
struct FUAFStateTreeContext
{
	GENERATED_BODY()
	virtual ~FUAFStateTreeContext() {};
	
	struct FPlaybackInfo
	{
		float PlaybackRatio = 0.f;
    	float TimeLeft = 0.f;
    	float Duration = 0.f;
    	bool bIsLooping = false;
	};

	virtual bool PushAssetOntoBlendStack(UE::UAF::FGraphAssetHandleConstView InAsset, const FAlphaBlendArgs& InBlendArguments, const UUAFBlendProfile* InBlendProfile) const { return false; };
    virtual void QueryPlaybackInfo(FPlaybackInfo& OutPlaybackInfo) const {};
	virtual FUAFAssetInstance* GetVariablesOwner() const { return nullptr; }
};


