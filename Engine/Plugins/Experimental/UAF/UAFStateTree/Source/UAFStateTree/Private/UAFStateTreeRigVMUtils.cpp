// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFStateTreeRigVMUtils.h"

#if WITH_EDITOR

#include "AnimNextFunctionHandle.h"
#include "AnimNextStateTreeEditorOnlyTypes.h"
#include "AnimNextRigVMAsset.h"
#include "Graph/AnimNextGraphContextData.h"
#include "Graph/AnimNextGraphInstance.h"
#include "StructUtils/PropertyBag.h"
#include "TraitCore/ExecutionContext.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"
#include "Script/UAFRigVMComponent.h"
#include "StateTree.h"
#include "StateTreeEditorData.h"
#include "StateTreeReference.h"
#include "Traits/CallFunction.h"
#include "UncookedOnlyUtils.h"
#include "Variables/AnimNextSharedVariables.h"

#endif // WITH_EDITOR

namespace UE::UAF::StateTree
{

#if WITH_EDITOR

	bool FUtils::IsBindingOutDated(FUAFCallFunctionInfo& CallFunctionInfo, FInstancedPropertyBag& DefaultValues, FInstancedPropertyBag& Parameters)
	{
		if (!Parameters.IsValid())
		{
			return false;
		}

		TConstArrayView<FPropertyBagPropertyDesc> CachedDescs = Parameters.GetPropertyBagStruct()->GetPropertyDescs();

		// Our function header might be out of date. Get latest
		CallFunctionInfo.FunctionHeader = FRigVMGraphFunctionHeader::FindGraphFunctionHeader(CallFunctionInfo.FunctionHeader.LibraryPointer);
		int32 RigVMFunctionBindableArgumentIndex = 0;

		for (const FRigVMGraphFunctionArgument& Argument : CallFunctionInfo.FunctionHeader.Arguments)
		{
			const bool bIsBindable = Argument.Direction == ERigVMPinDirection::Input || Argument.Direction == ERigVMPinDirection::Output;
			if (bIsBindable && !Argument.bIsInputVariable)
			{
				FString CppTypeString = Argument.CPPType.ToString();
				UObject* CppTypeObject = Argument.CPPTypeObject.Get();
				FPropertyBagPropertyDesc PropertyDesc = FRigVMMemoryStorageStruct::GeneratePropertyBagDescriptor(FRigVMPropertyDescription(Argument.Name, CppTypeString, CppTypeObject, Argument.DefaultValue));

				if (!CachedDescs.IsValidIndex(RigVMFunctionBindableArgumentIndex) || CachedDescs[RigVMFunctionBindableArgumentIndex] != PropertyDesc)
				{
					return true;
				}

				TValueOrError<FString, EPropertyBagResult> CachedDefaultValue = DefaultValues.GetValueSerializedString(Argument.Name);
				if (CachedDefaultValue.IsValid() && CachedDefaultValue.GetValue() != Argument.DefaultValue)
				{
					return true;
				}

				RigVMFunctionBindableArgumentIndex++;
			}
		}

		const int32 NumBindableArguments = RigVMFunctionBindableArgumentIndex;
		if (NumBindableArguments != CachedDescs.Num())
		{
			return true;
		}

		return false;
	}

	void FUtils::RegenerateParameterPropertyBag(FUAFCallFunctionInfo& CallFunctionInfo, FInstancedPropertyBag& DefaultValues, FInstancedPropertyBag& Parameters, int32& ResultIndex)
	{
		// Default values may not have been captured (Ex: old content). Always propagate values in this case.
		bool bDefaultValuesCached = DefaultValues.IsValid();

		// Before we regenerate, get a list of GUIDs for changed values. We will  migrate these over current defaults
		TArray<FGuid, TInlineAllocator<4>> ChangedValuePropertyIDs = {};
		ChangedValuePropertyIDs.Reserve(DefaultValues.GetNumPropertiesInBag());
		if (bDefaultValuesCached && ensure(DefaultValues.GetNumPropertiesInBag() == Parameters.GetNumPropertiesInBag()))
		{
			TConstArrayView<FPropertyBagPropertyDesc> CachedDescs = Parameters.GetPropertyBagStruct()->GetPropertyDescs();
			for (int32 CachedDescIndex = 0; CachedDescIndex < CachedDescs.Num(); CachedDescIndex++)
			{
				FName CachedPropertyName = CachedDescs[CachedDescIndex].Name;
				TValueOrError<FString, EPropertyBagResult> CachedValue = Parameters.GetValueSerializedString(CachedPropertyName);
				TValueOrError<FString, EPropertyBagResult> DefaultValue = DefaultValues.GetValueSerializedString(CachedPropertyName);

				if (ensure(CachedValue.IsValid() && DefaultValue.IsValid()))
				{
					if (CachedValue.GetValue() != DefaultValue.GetValue())
					{
						ChangedValuePropertyIDs.Add(CachedDescs[CachedDescIndex].ID);
					}
				}
			}
		}

		// Update defaults, then use that + above GUIDs to selectively update parameters for default values
		DefaultValues.Reset();

		TArray<FPropertyBagPropertyDesc, TInlineAllocator<4>> PropertyDescs;
		TArray<FStringView, TInlineAllocator<4>> DefaultValueStrings;
		PropertyDescs.Reserve(CallFunctionInfo.FunctionHeader.Arguments.Num());
		for (const FRigVMGraphFunctionArgument& Argument : CallFunctionInfo.FunctionHeader.Arguments)
		{
			const bool bIsBindable = Argument.Direction == ERigVMPinDirection::Input || Argument.Direction == ERigVMPinDirection::Output;
			if (bIsBindable && !Argument.bIsInputVariable)
			{
				FString CppTypeString = Argument.CPPType.ToString();
				UObject* CppTypeObject = Argument.CPPTypeObject.Get();
				FPropertyBagPropertyDesc PropertyDesc = FRigVMMemoryStorageStruct::GeneratePropertyBagDescriptor(FRigVMPropertyDescription(Argument.Name, CppTypeString, CppTypeObject, Argument.DefaultValue));
				if (Argument.Direction == ERigVMPinDirection::Output)
				{
					PropertyDesc.PropertyFlags &= ~CPF_Edit;	// Hide the results from the UI
				}

				PropertyDescs.Add(PropertyDesc);
				DefaultValueStrings.Add(Argument.DefaultValue);
			}
		}

		DefaultValues.AddProperties(PropertyDescs);
		for (int32 PropertyIndex = 0; PropertyIndex < PropertyDescs.Num(); PropertyIndex++)
		{
			DefaultValues.SetValueSerializedString(PropertyDescs[PropertyIndex].Name, DefaultValueStrings[PropertyIndex].GetData());
		}

		if (bDefaultValuesCached)
		{
			Parameters.MigrateToNewBagInstanceWithOverrides(DefaultValues, ChangedValuePropertyIDs);
		}
		else
		{
			Parameters.MigrateToNewBagInstance(DefaultValues);
		}
	}

#endif // WITH_EDITOR

}