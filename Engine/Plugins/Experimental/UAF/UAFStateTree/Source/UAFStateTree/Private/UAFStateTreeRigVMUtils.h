// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FInstancedPropertyBag;
struct FUAFCallFunctionInfo;

namespace UE::UAF::StateTree
{
	struct FUtils
	{
#if WITH_EDITOR
		/** True if parameters or default values no longer match the given function header. */
		static bool IsBindingOutDated(FUAFCallFunctionInfo& CallFunctionInfo, FInstancedPropertyBag& DefaultValues, FInstancedPropertyBag& Parameters);

		/** Repopulates parameters and default values with the given function header. */
		static void RegenerateParameterPropertyBag(FUAFCallFunctionInfo& CallFunctionInfo, FInstancedPropertyBag& DefaultValues, FInstancedPropertyBag& Parameters, int32& ResultIndex);
#endif // WITH_EDITOR
	};
}