// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFStateTreeTraitContext.h"
#include "Factory/AnimGraphFactory.h"
#include "TraitCore/NodeInstance.h"
#include "TraitCore/TraitStackBinding.h"
#include "TraitInterfaces/IBlendStack.h"
#include "TraitInterfaces/ITimeline.h"

bool FUAFStateTreeTraitContext::PushAssetOntoBlendStack(UE::UAF::FGraphAssetHandleConstView InAsset, const FAlphaBlendArgs& InBlendArguments, const UUAFBlendProfile* BlendProfile) const
{
	using namespace UE::UAF;

	TTraitBinding<IBlendStack> BlendStackBinding;
	if(!Binding->GetStackInterface<IBlendStack>(BlendStackBinding))
	{
		return false;
	}

	// Instantiate/get a graph from the params
	FAnimNextFactoryParams Params = FAnimGraphFactory::GetDefaultParamsForAsset(InAsset);
	const UUAFAnimGraph* AnimationGraph = FAnimGraphFactory::GetOrBuildGraph(InAsset);
	if(AnimationGraph == nullptr)
	{
		return false;
	}

	IBlendStack::FGraphRequest NewGraphRequest;
	NewGraphRequest.BlendArgs = InBlendArguments;
	// Todo: change to asset handle?
	TArray<const UObject*> Objects;
	InAsset.GetPtr()->GetObjectReferences(Objects);
	NewGraphRequest.FactoryObject = Objects.Num() > 0 ? Objects[0] : nullptr;
	NewGraphRequest.AnimationGraph = AnimationGraph;
	NewGraphRequest.FactoryParams = Params;
	NewGraphRequest.BlendProfile = BlendProfile;

	BlendStackBinding.PushGraph(*Context, MoveTemp(NewGraphRequest));

	return true;
}

void FUAFStateTreeTraitContext::QueryPlaybackInfo(FPlaybackInfo& OutPlaybackInfo) const
{
	UE::UAF::TTraitBinding<UE::UAF::ITimeline> Timeline;
	if (Binding->GetStackInterface<UE::UAF::ITimeline>(Timeline))
	{
		UE::UAF::FTimelineState TimelineState;
		if (Timeline.GetState(*Context, TimelineState))
		{
			OutPlaybackInfo.PlaybackRatio = TimelineState.GetPositionRatio();
			OutPlaybackInfo.TimeLeft = TimelineState.GetTimeLeft();
			OutPlaybackInfo.Duration = TimelineState.GetDuration();
			OutPlaybackInfo.bIsLooping = TimelineState.IsLooping();
		}
	}
}

FUAFAssetInstance* FUAFStateTreeTraitContext::GetVariablesOwner() const
{
	return &Binding->GetTraitPtr().GetNodeInstance()->GetOwner();
}