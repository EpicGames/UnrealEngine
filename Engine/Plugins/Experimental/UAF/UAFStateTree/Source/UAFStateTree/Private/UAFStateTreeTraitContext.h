// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFStateTreeContext.h"
#include "AnimStateTreeTrait.h"
#include "UAFStateTreeTraitContext.generated.h"

namespace UE::UAF
{
	struct FTraitBinding;
	struct FExecutionContext;
	struct FStateTreeTrait;
}

class UUAFAnimGraph;
struct FUAFAssetInstance;

USTRUCT()
struct FUAFStateTreeTraitContext : public FUAFStateTreeContext
{	
	GENERATED_BODY()
	
	friend UE::UAF::FStateTreeTrait;
	friend UE::UAF::FStateTreeTrait::FInstanceData;

	FUAFStateTreeTraitContext() = default;

	FUAFStateTreeTraitContext(UE::UAF::FExecutionContext& InContext, const UE::UAF::FTraitBinding& InBinding)
		: Context(&InContext)
		, Binding(&InBinding)
	{}
	
	virtual bool PushAssetOntoBlendStack(UE::UAF::FGraphAssetHandleConstView InAsset, const FAlphaBlendArgs& InBlendArguments, const UUAFBlendProfile* BlendProfile) const override;
	
    virtual void QueryPlaybackInfo(FPlaybackInfo& OutPlaybackInfo) const override;
	virtual FUAFAssetInstance* GetVariablesOwner() const override;

	UE::UAF::FExecutionContext& GetContext() const
	{
		check(Context != nullptr);
		return *Context;
	}

	const UE::UAF::FTraitBinding& GetBinding() const
	{
		check(Binding != nullptr);
		return *Binding;
	}

private:

	UE::UAF::FExecutionContext* Context = nullptr;
	const UE::UAF::FTraitBinding* Binding = nullptr;
};