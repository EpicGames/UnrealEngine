// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFStateTree : ModuleRules
	{
		public UAFStateTree(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UAFAnimGraph",
					"UAFAnimNode",
					"RigVM"
				});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"StateTreeModule",
					"Engine",
					"UAF",
					"HierarchyTableRuntime",
					"HierarchyTableAnimationRuntime",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UAFAnimGraph",
						"UAFAnimGraphUncookedOnly",
						"UAFEditor",
						"UAFUncookedOnly",
						"RigVMDeveloper",
						"StateTreeEditorModule",
						"UnrealEd"
					});
			}
		}
	}
}