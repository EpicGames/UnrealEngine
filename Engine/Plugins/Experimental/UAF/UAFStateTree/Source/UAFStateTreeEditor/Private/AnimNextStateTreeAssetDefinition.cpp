// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeAssetDefinition.h"
#include "IWorkspaceEditorModule.h"
#include "Workspace/AnimNextWorkspaceFactory.h"
#include "Modules/ModuleManager.h"

EAssetCommandResult UAssetDefinition_AnimNextStateTree::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::UAF::Editor;
	using namespace UE::Workspace;

	for (UAnimNextStateTree* Asset : OpenArgs.LoadObjects<UAnimNextStateTree>())
	{
		IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<IWorkspaceEditorModule>("WorkspaceEditor");
		WorkspaceEditorModule.OpenWorkspaceForObject(Asset, EOpenWorkspaceMethod::Default, UAnimNextWorkspaceFactory::StaticClass());
	}

	return EAssetCommandResult::Handled;
}
