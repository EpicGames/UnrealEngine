// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeEditorHost.h"

#include "AnimNextStateTree.h"
#include "IAnimNextEditorModule.h"
#include "StateTree.h"
#include "StateTreeEditorWorkspaceTabHost.h"

#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"

void FAnimNextStateTreeEditorHost::Init(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWeakWorkspaceEditor)
{
	WeakWorkspaceEditor = InWeakWorkspaceEditor;
	TabHost = MakeShared<UE::StateTreeEditor::FWorkspaceTabHost>();
	
	TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedEditor = InWeakWorkspaceEditor.Pin();
	check(SharedEditor.IsValid());
	SharedEditor->OnFocusedDocumentChanged().AddSP(this, &FAnimNextStateTreeEditorHost::OnWorkspaceFocusedDocumentChanged);
}

UStateTree* FAnimNextStateTreeEditorHost::GetStateTree() const
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		const UE::Workspace::FWorkspaceDocument& Document = SharedWorkspaceEditor->GetFocusedDocumentOfClass(UAnimNextStateTree::StaticClass());
		const TObjectPtr<UAnimNextStateTree> AnimNextStateTreePtr = Document.GetTypedObject<UAnimNextStateTree>();
		return AnimNextStateTreePtr ? AnimNextStateTreePtr->StateTree : nullptr;
	}	

	return nullptr;
}

FName FAnimNextStateTreeEditorHost::GetCompilerLogName() const
{
	return UE::UAF::Editor::LogListingName;
}

FName FAnimNextStateTreeEditorHost::GetCompilerTabName() const
{
	return UE::UAF::Editor::CompilerResultsTabName;
}

bool FAnimNextStateTreeEditorHost::ShouldShowCompileButton() const
{
	return false;
}

bool FAnimNextStateTreeEditorHost::CanToolkitSpawnWorkspaceTab() const
{
	return true;
}

void FAnimNextStateTreeEditorHost::OnWorkspaceFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument) const
{
	TObjectPtr<UAnimNextStateTree> InStateTree = InDocument.GetTypedObject<UAnimNextStateTree>();

	if (TStrongObjectPtr<UAnimNextStateTree> LastStateTreePinned = WeakLastStateTree.Pin())
	{
		if (LastStateTreePinned != InStateTree)
		{
			OnStateTreeChangedDelegate.Broadcast();
		}
	}
	else if (InStateTree)
	{
		OnStateTreeChangedDelegate.Broadcast();
	}

	// Always set the last state tree as incoming object after cast. If it's null that will clear the last state tree
	WeakLastStateTree = InStateTree;
}

FSimpleMulticastDelegate& FAnimNextStateTreeEditorHost::OnStateTreeChanged()
{
	return OnStateTreeChangedDelegate;
}

TSharedPtr<IDetailsView> FAnimNextStateTreeEditorHost::GetAssetDetailsView()
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedEditor = WeakWorkspaceEditor.Pin())
	{
		return SharedEditor->GetDetailsView();
	}
	
	return nullptr;
}

TSharedPtr<IDetailsView> FAnimNextStateTreeEditorHost::GetDetailsView()
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedEditor = WeakWorkspaceEditor.Pin())
	{
		return SharedEditor->GetDetailsView();
	}
	
	return nullptr;
}

TSharedPtr<UE::StateTreeEditor::FWorkspaceTabHost> FAnimNextStateTreeEditorHost::GetTabHost() const
{
	return TabHost;
}
