// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStateTreeEditorHost.h"

namespace UE::Workspace
{
	class IWorkspaceEditor;
	struct FWorkspaceDocument;
}

class UAnimNextStateTree;

class FAnimNextStateTreeEditorHost : public IStateTreeEditorHost
{
public:
	void Init(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWeakWorkspaceEditor);

	// IStateTreeEditorHost overrides
	virtual UStateTree* GetStateTree() const override;
	virtual FName GetCompilerLogName() const override;
	virtual FName GetCompilerTabName() const override;
	virtual bool ShouldShowCompileButton() const override;
	virtual bool CanToolkitSpawnWorkspaceTab() const override;
	virtual FSimpleMulticastDelegate& OnStateTreeChanged() override;
	virtual TSharedPtr<IDetailsView> GetAssetDetailsView() override;
	virtual TSharedPtr<IDetailsView> GetDetailsView() override;
	virtual TSharedPtr<UE::StateTreeEditor::FWorkspaceTabHost> GetTabHost() const override;

protected:
	void OnWorkspaceFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument) const;

	TWeakPtr<UE::Workspace::IWorkspaceEditor> WeakWorkspaceEditor;	
	FSimpleMulticastDelegate OnStateTreeChangedDelegate;
	TSharedPtr<UE::StateTreeEditor::FWorkspaceTabHost> TabHost;

	/** Last state tree, cached to avoid changing debugger & losing context more than needed */
	mutable TWeakObjectPtr<UAnimNextStateTree> WeakLastStateTree;
};