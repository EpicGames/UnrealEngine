// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeEditorModule.h"

#include "AnimNextStateTree.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "StateTree.h"
#include "AnimNextStateTreeWorkspaceExports.h"
#include "IAnimNextEditorModule.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "StateTreeEditorMode.h"
#include "WorkspaceItemMenuContext.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "AnimNextStateTreeEditorHost.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditorStyle.h"
#include "UncookedOnlyUtils.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "ToolMenus.h"
#include "Framework/MultiBox/ToolMenuBase.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AnimNextStateTreeEditorData.h"
#include "AnimNextStateTreeSchema.h"
#include "StateTreeAssetCompilationHandler.h"
#include "StateTreeEditorModule.h"
#include "UAFStateTreeEditorSchema.h"
#include "UAFAnimNodeEditor/Public/UAFAnimNodeDataFactory.h"
#include "AnimNode/UAFStateTreeNode.h"

#define LOCTEXT_NAMESPACE "AnimNextStateTreeEditorModule"

namespace UE::UAF::StateTree
{
void FAnimNextStateTreeEditorModule::StartupModule()
{
	// Register StateTree as supported asset in AnimNext workspaces
	Editor::IAnimNextEditorModule& AnimNextEditorModule = FModuleManager::Get().LoadModuleChecked<Editor::IAnimNextEditorModule>("UAFEditor");	
	AnimNextEditorModule.AddWorkspaceSupportedAssetClass(UAnimNextStateTree::StaticClass()->GetClassPathName());
	AnimNextEditorModule.RegisterAssetCompilationHandler(UAnimNextStateTree::StaticClass()->GetClassPathName(), Editor::FAssetCompilationHandlerFactoryDelegate::CreateLambda([](UObject* InAsset)
	{
		TSharedRef<FStateTreeAssetCompilationHandler> CompilationHandler = MakeShared<FStateTreeAssetCompilationHandler>(InAsset);
		CompilationHandler->Initialize();
		return CompilationHandler;
	}));

	// Extend Workspace Editor layout to deal with StateTreeEditorMode tabs
	Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
	WorkspaceEditorModule.OnExtendTabs().AddLambda([](FLayoutExtender& InLayoutExtender, TSharedPtr<UE::Workspace::IWorkspaceEditor> InEditorPtr)
	{
		FTabManager::FTab TreeOutlinerTab(FTabId(UAssetEditorUISubsystem::TopLeftTabID), ETabState::ClosedTab);
		InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::TopLeftDocumentArea), ELayoutExtensionPosition::After, TreeOutlinerTab);

		FTabManager::FTab StatisticsTab(FTabId(UAssetEditorUISubsystem::BottomRightTabID), ETabState::ClosedTab);
		InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::BottomMiddleDocumentArea), ELayoutExtensionPosition::After, StatisticsTab);

		FTabManager::FTab DebuggerTab(FTabId(UAssetEditorUISubsystem::TopRightTabID), ETabState::ClosedTab);
		InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::BottomMiddleDocumentArea), ELayoutExtensionPosition::After, DebuggerTab);		
	});

	WorkspaceEditorModule.OnExtendToolMenuContext().AddLambda([](const TWeakPtr<Workspace::IWorkspaceEditor>& InWorkspaceEditor, FToolMenuContext& InContext)
	{
		if(!InWorkspaceEditor.Pin()->GetEditorModeManager().IsModeActive(UStateTreeEditorMode::EM_StateTree))
		{
			UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
			ProfileContext->ActiveProfiles.Add( TEXT("StateTreeEditModeDisabledProfile") );
			InContext.AddObject(ProfileContext);
		}
	});

	
	// --- AnimNextStateTree ---
	Workspace::FObjectDocumentArgs StateTreeDocumentArgs(
		Workspace::FOnMakeDocumentWidget::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext)-> TSharedRef<SWidget>
		{
			UAnimNextStateTree* AnimNextStateTree = InContext.Document.GetTypedObject<UAnimNextStateTree>();
			UStateTree* StateTree = AnimNextStateTree->StateTree;

			TWeakPtr<Workspace::IWorkspaceEditor> WeakWorkspaceEditor = InContext.WorkspaceEditor;
			UContextObjectStore* ContextStore = InContext.WorkspaceEditor->GetEditorModeManager().GetInteractiveToolsContext()->ContextObjectStore;
			UStateTreeEditorContext* StateTreeEditorContext = ContextStore->FindContext<UStateTreeEditorContext>();
			if (!StateTreeEditorContext)
			{
				StateTreeEditorContext = NewObject<UStateTreeEditorContext>();
				TSharedPtr<FAnimNextStateTreeEditorHost> Host = MakeShared<FAnimNextStateTreeEditorHost>();
				Host->Init(WeakWorkspaceEditor);
				StateTreeEditorContext->EditorHostInterface = Host;
				ContextStore->AddContextObject(StateTreeEditorContext);
			}

			if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{
				TSharedRef<FStateTreeViewModel> StateTreeViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(StateTree);
				TSharedRef<SWidget> StateTreeViewWidget = StateTreeEditingSubsystem->GetStateTreeView(StateTreeViewModel, InContext.WorkspaceEditor->GetToolkitCommands());

				TWeakPtr<SWidget> WeakStateTreeViewWidget = StateTreeViewWidget;
				TWeakPtr<FStateTreeViewModel> WeakViewModel = StateTreeViewModel;

				// @TODO: This works but it would be more idiomatic to have the remove occur at a point of de-initialization. This allows for the context object to be destroyed between user sessions without leaking delegate callbacks.
				StateTreeViewModel->GetOnSelectionChanged().RemoveAll(&InContext.WorkspaceEditor.Get());
				StateTreeViewModel->GetOnSelectionChanged().AddSPLambda(&InContext.WorkspaceEditor.Get(), [WeakStateTreeViewWidget, WeakWorkspaceEditor, WeakViewModel](const TArray<TWeakObjectPtr<UStateTreeState>>& SelectedStates)
				{
					if (TSharedPtr<Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
					{
						TArray<UObject*> Selected;
						for (const TWeakObjectPtr<UStateTreeState>& WeakState : SelectedStates)
						{
							if (UStateTreeState* State = WeakState.Get())
							{
								Selected.Add(State);
							}
						}

						SharedWorkspaceEditor->SetGlobalSelection(WeakStateTreeViewWidget, UE::Workspace::FOnClearGlobalSelection::CreateLambda([WeakViewModel, WeakStateTreeViewWidget]()
						{ 
							if (WeakStateTreeViewWidget.IsValid())
							{
								if (const TSharedPtr<FStateTreeViewModel> SharedViewModel = WeakViewModel.Pin())
								{
									SharedViewModel->ClearSelection();
								}
							}
						}));						
						SharedWorkspaceEditor->SetDetailsObjects(Selected);
					}					
				});

				return SNew(SVerticalBox)				
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					StateTreeViewWidget
				];
			}
						
			return SNullWidget::NullWidget;
		}
	), Workspace::WorkspaceTabs::TopMiddleDocumentArea);
	
	StateTreeDocumentArgs.OnGetTabName = Workspace::FOnGetTabName::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext)
	{
		UAnimNextStateTree* AnimNextStateTree = InContext.Document.GetTypedObject<UAnimNextStateTree>();
		return FText::FromName(AnimNextStateTree->GetFName());
	});

	StateTreeDocumentArgs.DocumentEditorMode = UStateTreeEditorMode::EM_StateTree;

	StateTreeDocumentArgs.OnGetDocumentBreadcrumbTrail = Workspace::FOnGetDocumentBreadcrumbTrail::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, TArray<TSharedPtr<Workspace::FWorkspaceBreadcrumb>>& OutBreadcrumbs)
	{
		if (UAnimNextStateTree* AnimNextStateTree = InContext.Document.GetTypedObject<UAnimNextStateTree>())
		{
			const TSharedPtr<Workspace::FWorkspaceBreadcrumb>& GraphCrumb = OutBreadcrumbs.Add_GetRef(MakeShared<Workspace::FWorkspaceBreadcrumb>());
			GraphCrumb->OnGetLabel = Workspace::FWorkspaceBreadcrumb::FOnGetBreadcrumbLabel::CreateLambda([StateTreeName = AnimNextStateTree->GetFName()]{ return FText::FromName(StateTreeName); });

			GraphCrumb->CanSave = Workspace::FWorkspaceBreadcrumb::FCanSaveBreadcrumb::CreateLambda(
				[AnimNextStateTree]
				{
					return AnimNextStateTree->GetPackage()->IsDirty();
				}
			);

			TWeakPtr<Workspace::IWorkspaceEditor> WeakWorkspaceEditor = InContext.WorkspaceEditor;
			TWeakObjectPtr<UAnimNextStateTree> WeakStateTree = AnimNextStateTree;
			GraphCrumb->OnClicked = Workspace::FWorkspaceBreadcrumb::FOnBreadcrumbClicked::CreateLambda(
			[WeakWorkspaceEditor, Export = InContext.Document.Export]
			{
				if (const TSharedPtr<Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
				{
					SharedWorkspaceEditor->OpenExports({ Export });
				}
			}
		);
		}
	});

	StateTreeDocumentArgs.OnGetTabIcon = Workspace::FOnGetTabIcon::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext)
	{
		return FAppStyle::GetBrush(TEXT("ClassIcon.Default"));
	});
	
	WorkspaceEditorModule.RegisterObjectDocumentType(UAnimNextStateTree::StaticClass()->GetClassPathName(), StateTreeDocumentArgs);

	class FStateTreeAssetOutlinerItemDetails : public Workspace::IWorkspaceOutlinerItemDetails
	{
	public:
		virtual ~FStateTreeAssetOutlinerItemDetails() override = default;
		virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override
		{
			if (Export.GetData().GetScriptStruct() == FAnimNextStateTreeOutlinerData::StaticStruct())
			{
				return FAppStyle::GetBrush(TEXT("ClassIcon.Default"));
			}
			else if (Export.GetData().GetScriptStruct() == FAnimNextStateTreeStateOutlinerData::StaticStruct())
			{
				const FAnimNextStateTreeStateOutlinerData& Data = Export.GetData().Get<FAnimNextStateTreeStateOutlinerData>();
				return FStateTreeEditorStyle::GetBrushForSelectionBehaviorType(Data.SelectionBehavior, !Data.bIsLeafState, Data.Type);
			}
			
			return nullptr;
		}

		virtual FSlateColor GetItemColor(const FWorkspaceOutlinerItemExport& Export) const override
		{
			if (Export.GetData().GetScriptStruct() == FAnimNextStateTreeStateOutlinerData::StaticStruct())
			{
				const FAnimNextStateTreeStateOutlinerData& Data = Export.GetData().Get<FAnimNextStateTreeStateOutlinerData>();
				return Data.Color;
			}
			
			return FSlateColor::UseForeground();
		}

		static void SelectStateExports(TConstArrayView<FWorkspaceOutlinerItemExport> StateExports)
		{
			if(UStateTreeEditingSubsystem* EditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
			{
				UAnimNextStateTree* SelectionStateTree = [StateExports]() -> UAnimNextStateTree*
				{
					UAnimNextStateTree* FoundTree = nullptr;

					for (const FWorkspaceOutlinerItemExport& SelectedExport : StateExports)
					{
						if (UAnimNextStateTree* LoadedStateTree = Cast<UAnimNextStateTree>(SelectedExport.GetFirstAssetPath().ResolveObject()))
						{
							if (FoundTree == nullptr)
							{
								FoundTree = LoadedStateTree;
							}
							if (FoundTree != LoadedStateTree)
							{
								FoundTree = nullptr;
								break;
							}
						}
					}

					return FoundTree;
				}();

				if (SelectionStateTree)
				{
					TSharedPtr<FStateTreeViewModel> ViewModel = EditingSubsystem->FindOrAddViewModel(SelectionStateTree->StateTree.Get());
					check(ViewModel.IsValid());
			
					TArray<TWeakObjectPtr<UStateTreeState>> ToBeSelectedStates;
					for (const FWorkspaceOutlinerItemExport& SelectedExport : StateExports)
					{
						if (SelectedExport.GetData().GetScriptStruct() == FAnimNextStateTreeStateOutlinerData::StaticStruct())
						{
							if (UAnimNextStateTree* LoadedStateTree = Cast<UAnimNextStateTree>(SelectedExport.GetFirstAssetPath().ResolveObject()))
							{
								const FAnimNextStateTreeStateOutlinerData& StateData = SelectedExport.GetData().Get<FAnimNextStateTreeStateOutlinerData>();		
								ensure(LoadedStateTree == SelectionStateTree);
								if (UStateTreeState* State = ViewModel->GetMutableStateByID(StateData.StateId))
								{
									ToBeSelectedStates.Add(State);
								}
							}
						}
					}

					ViewModel->SetSelection(ToBeSelectedStates);
				}
			}	
		}
		
		virtual bool HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const override
		{
			const UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
			const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>(); 
			if (WorkspaceItemContext && AssetEditorContext)
			{				
				if(const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin()))
				{
					const uint32 NumSelectedExports = WorkspaceItemContext->SelectedExports.Num();				
					if (NumSelectedExports == 1)
					{
						const FWorkspaceOutlinerItemExport& SelectedExport = WorkspaceItemContext->SelectedExports[0].GetResolvedExport();

						if(SelectedExport.GetData().GetScriptStruct() == FAnimNextStateTreeOutlinerData::StaticStruct())
						{
							if (UAnimNextStateTree* LoadedStateTree = SelectedExport.GetFirstAssetOfType<UAnimNextStateTree>())
							{
								WorkspaceEditor->OpenExports({ WorkspaceItemContext->SelectedExports[0] });

								return true;
							}
						}
						else if (SelectedExport.GetData().GetScriptStruct() == FAnimNextStateTreeStateOutlinerData::StaticStruct())
						{
							if (UAnimNextStateTree* SelectionStateTree = SelectedExport.GetFirstAssetOfType<UAnimNextStateTree>())
							{
								// First open editor for state-tree
								WorkspaceEditor->OpenExports({ WorkspaceItemContext->SelectedExports[0] });

								// Then select the double-clicked state
								SelectStateExports(WorkspaceItemContext->SelectedExports);
								
								return true;
							}								
						}							
					}					
				}			
			}

			return false;
		}
		
		virtual bool HandleSelected(const FToolMenuContext& ToolMenuContext) const override
		{
			const UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
			const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>(); 
			if (WorkspaceItemContext && AssetEditorContext)
			{				
				if(const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin()))
				{
					const uint32 NumSelectedExports = WorkspaceItemContext->SelectedExports.Num();
					if (NumSelectedExports >= 1)
					{
						// Selecting AnimStateTree asset itself
						if (NumSelectedExports == 1 && WorkspaceItemContext->SelectedExports[0].GetResolvedExport().GetData().GetScriptStruct() == FAnimNextStateTreeOutlinerData::StaticStruct())
						{
							const FWorkspaceOutlinerItemExport& SelectedExport = WorkspaceItemContext->SelectedExports[0].GetResolvedExport();
							if (UAnimNextStateTree* LoadedStateTree = Cast<UAnimNextStateTree>(SelectedExport.GetFirstAssetPath().ResolveObject()))
							{
								WorkspaceEditor->SetDetailsObjects({ LoadedStateTree->StateTree->EditorData } );
								return true;
							}
						}

						SelectStateExports(WorkspaceItemContext->SelectedExports);
					}
				}			
			}
			return false;
		}
	};
	
	TSharedPtr<FStateTreeAssetOutlinerItemDetails> StateItemDetails = MakeShareable<FStateTreeAssetOutlinerItemDetails>(new FStateTreeAssetOutlinerItemDetails());
	WorkspaceEditorModule.RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextStateTreeOutlinerData::StaticStruct()->GetFName()), StaticCastSharedPtr<UE::Workspace::IWorkspaceOutlinerItemDetails>(StateItemDetails));	
	WorkspaceEditorModule.RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextStateTreeStateOutlinerData::StaticStruct()->GetFName()), StaticCastSharedPtr<UE::Workspace::IWorkspaceOutlinerItemDetails>(StateItemDetails));

	FStateTreeEditorModule& StateTreeEditorModule = FStateTreeEditorModule::GetModule();
	StateTreeEditorModule.RegisterEditorDataClass(UStateTreeAnimNextSchema::StaticClass(), UAnimNextStateTreeTreeEditorData::StaticClass());
	StateTreeEditorModule.RegisterEditorSchemaClass(UStateTreeAnimNextSchema::StaticClass(), UUAFStateTreeEditorSchema::StaticClass());

	StateTreeClassPath = FUAFAnimNodeDataFactory::RegisterAsset<FUAFStateTreeNodeData, UAnimNextStateTree>(
	[](UAnimNextStateTree* StateTree)
		{
			FUAFStateTreeNodeData StateTreeNode;
			StateTreeNode.SetStateTree(StateTree);
			return StateTreeNode;
		});
}

void FAnimNextStateTreeEditorModule::ShutdownModule()
{
	if (UObjectInitialized())
	{
		if(Workspace::IWorkspaceEditorModule* WorkspaceEditorModule = FModuleManager::Get().GetModulePtr<Workspace::IWorkspaceEditorModule>("WorkspaceEditor"))
		{
				WorkspaceEditorModule->UnregisterObjectDocumentType(UAnimNextStateTree::StaticClass()->GetClassPathName());
				WorkspaceEditorModule->UnregisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextStateTreeOutlinerData::StaticStruct()->GetFName()));
				WorkspaceEditorModule->UnregisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextStateTreeStateOutlinerData::StaticStruct()->GetFName()));
		}
	
		if( Editor::IAnimNextEditorModule* AnimNextEditorModule = FModuleManager::Get().GetModulePtr<Editor::IAnimNextEditorModule>("UAFEditor"))
		{
			AnimNextEditorModule->RemoveWorkspaceSupportedAssetClass(UStateTree::StaticClass()->GetClassPathName());
			AnimNextEditorModule->UnregisterAssetCompilationHandler(UAnimNextStateTree::StaticClass()->GetClassPathName());
		}

		if(FStateTreeEditorModule* StateTreeEditorModule = FStateTreeEditorModule::GetModulePtr())
		{
			StateTreeEditorModule->UnregisterEditorDataClass(UStateTreeAnimNextSchema::StaticClass());
			StateTreeEditorModule->UnregisterEditorSchemaClass(UStateTreeAnimNextSchema::StaticClass());
		}
    }
    
    FUAFAnimNodeDataFactory::UnregisterAsset(StateTreeClassPath);
}

IMPLEMENT_MODULE(FAnimNextStateTreeEditorModule, UAFStateTreeEditor)

}

#undef LOCTEXT_NAMESPACE // "AnimNextStateTreeEditorModule"
