// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAnimNextStateTreeEditorModule.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::UAF::StateTree
{
class FAnimNextStateTreeEditorModule : public IAnimNextStateTreeEditorModule
{
public:	
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
private:
	FTopLevelAssetPath StateTreeClassPath;
};
}
