// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeFactory.h"

#include "UAFCompilationScope.h"
#include "AnimNextStateTree.h"
#include "AnimNextStateTreeSchema.h"
#include "AnimNextStateTree_EditorData.h"
#include "StateTreeFactory.h"
#include "AnimStateTreeTrait.h"

UAnimNextStateTreeFactory::UAnimNextStateTreeFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAnimNextStateTree::StaticClass();
}

bool UAnimNextStateTreeFactory::ConfigureProperties()
{
	return true;
}

UObject* UAnimNextStateTreeFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if(InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	UAnimNextStateTree* NewStateTree = NewObject<UAnimNextStateTree>(InParent, Class, Name, FlagsToUse);
	// Create internal editor data
	UAnimNextStateTree_EditorData* EditorData = NewObject<UAnimNextStateTree_EditorData>(NewStateTree, FName(Name.ToString() + "_StateTree_EditorData"), RF_Transactional);
	NewStateTree->EditorData = EditorData;

	// Create internal state tree
	UStateTreeFactory* StateTreeFactory = NewObject<UStateTreeFactory>(GetTransientPackage());
	StateTreeFactory->SetSchemaClass(UStateTreeAnimNextSchema::StaticClass());

	UStateTree* StateTree = CastChecked<UStateTree>(StateTreeFactory->FactoryCreateNew(UStateTree::StaticClass(), NewStateTree, FName(Name.ToString() + "_StateTree"), RF_Transactional, nullptr, nullptr));
	NewStateTree->StateTree = StateTree;

	EditorData->Initialize(/*bRecompileVM*/false);

	// Compile the initial skeleton
	UE::UAF::UncookedOnly::Compilation::RequestAssetCompilation(NewStateTree);
	
	check(!EditorData->bErrorsDuringCompilation);

	return NewStateTree;
}
