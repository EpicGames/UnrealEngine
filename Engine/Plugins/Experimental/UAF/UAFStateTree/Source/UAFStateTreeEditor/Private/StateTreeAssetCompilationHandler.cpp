// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateTreeAssetCompilationHandler.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "UAFCompilationScope.h"
#include "AnimNextStateTree.h"
#include "AnimNextStateTree_EditorData.h"
#include "Editor.h"
#include "StateTree.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditingSubsystem.h"
#include "UncookedOnlyUtils.h"

namespace UE::UAF::StateTree
{

FStateTreeAssetCompilationHandler::FStateTreeAssetCompilationHandler(UObject* InAsset)
	: FAssetCompilationHandler(InAsset)
{
	UAnimNextStateTree* AnimNextStateTree = CastChecked<UAnimNextStateTree>(InAsset);
	UStateTree* StateTree = AnimNextStateTree->StateTree;
	check(AnimNextStateTree->StateTree);
	CachedStateTree = AnimNextStateTree->StateTree;
	UStateTreeEditingSubsystem::ValidateStateTree(AnimNextStateTree->StateTree);
}

void FStateTreeAssetCompilationHandler::Initialize()
{
	if (UStateTreeEditingSubsystem* StateTreeEditingSubsystem = GEditor->GetEditorSubsystem<UStateTreeEditingSubsystem>())
	{
		TSharedRef<FStateTreeViewModel> ViewModel = StateTreeEditingSubsystem->FindOrAddViewModel(CachedStateTree.Get());
		ViewModel->GetOnAssetChanged().AddSP(this, &FStateTreeAssetCompilationHandler::HandleAssetChanged);
		ViewModel->GetOnStatesChanged().AddSP(this, &FStateTreeAssetCompilationHandler::HandleStatesChanged);
		ViewModel->GetOnStateAdded().AddSP(this, &FStateTreeAssetCompilationHandler::HandleStateAdded);
		ViewModel->GetOnStatesRemoved().AddSP(this, &FStateTreeAssetCompilationHandler::HandleStatesRemoved);
		ViewModel->GetOnStatesMoved().AddSP(this, &FStateTreeAssetCompilationHandler::HandleStatesMoved);
	}
}

void FStateTreeAssetCompilationHandler::Compile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset)
{
	UAnimNextStateTree* AnimNextStateTree = Cast<UAnimNextStateTree>(InAsset);
	if (AnimNextStateTree == nullptr)
	{
		return;
	}

	UAnimNextStateTree_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UAnimNextStateTree_EditorData>(AnimNextStateTree);
	if (EditorData == nullptr)
	{
		return;
	}

	UE::UAF::UncookedOnly::FCompilationScope CompileScope(AnimNextStateTree);
	FCompilerResultsLog& CompileLog = CompileScope.GetLogForObject(AnimNextStateTree);

	// We can't recompile state tree while in PIE
	if (!GEditor->IsPlaySessionInProgress())
	{
		// First compile the state tree
		FStateTreeCompilerLog Log;
		UStateTreeEditingSubsystem::CompileStateTree(AnimNextStateTree->StateTree, Log);

		// Add to messages
		for(const TSharedRef<FTokenizedMessage>& Message : Log.ToTokenizedMessages())
		{
			CompileLog.AddTokenizedMessage(Message);
		}
	}
	else
	{
		CompileLog.Note(*FString::Printf(TEXT("StateTree '%s' compilation was skipped because PIE was running"), *InAsset->GetName()));
	}

	// Call super to compile the RigVM Asset 
	UE::UAF::Editor::FAssetCompilationHandler::Compile(InWorkspaceEditor, InAsset);
}

UE::UAF::Editor::ECompileStatus FStateTreeAssetCompilationHandler::GetCompileStatus(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const
{
	const UAnimNextStateTree* AnimNextStateTree = Cast<UAnimNextStateTree>(InAsset);
	if (AnimNextStateTree == nullptr)
	{
		return UE::UAF::Editor::ECompileStatus::Unknown;
	}

	UE::UAF::Editor::ECompileStatus RigVMAssetStatus = UE::UAF::Editor::FAssetCompilationHandler::GetCompileStatus(InWorkspaceEditor, InAsset);
	UE::UAF::Editor::ECompileStatus StateTreeAssetStatus = UE::UAF::Editor::ECompileStatus::Unknown;
	if (AnimNextStateTree->StateTree)
	{
		if (AnimNextStateTree->StateTree->IsEditorDataDirty())
		{
			StateTreeAssetStatus = UE::UAF::Editor::ECompileStatus::Dirty;
		}
		else if (!AnimNextStateTree->StateTree->IsReadyToRun())
		{
			StateTreeAssetStatus = UE::UAF::Editor::ECompileStatus::Error;
		}
		else
		{
			StateTreeAssetStatus = UE::UAF::Editor::ECompileStatus::UpToDate;
		}
	}

	return FMath::Max(RigVMAssetStatus, StateTreeAssetStatus);
}

void FStateTreeAssetCompilationHandler::UpdateCachedInfo()
{
	UStateTree* StateTree = CachedStateTree.Get();
	if(StateTree == nullptr)
	{
		return;
	}

	UStateTreeEditingSubsystem::ValidateStateTree(StateTree);
	OnCompileStatusChanged().ExecuteIfBound();
}

void FStateTreeAssetCompilationHandler::HandleAssetChanged()
{
	UpdateCachedInfo();
}

void FStateTreeAssetCompilationHandler::HandleStatesChanged(const TSet<UStateTreeState*>& /*AffectedStates*/, const FPropertyChangedEvent& /*PropertyChangedEvent*/)
{
	UpdateCachedInfo();
}

void FStateTreeAssetCompilationHandler::HandleStateAdded(UStateTreeState* /*ParentState*/, UStateTreeState* /*NewState*/)
{
	UpdateCachedInfo();
}

void FStateTreeAssetCompilationHandler::HandleStatesRemoved(const TSet<UStateTreeState*>& /*AffectedParents*/)
{
	UpdateCachedInfo();
}

void FStateTreeAssetCompilationHandler::HandleStatesMoved(const TSet<UStateTreeState*>& /*AffectedParents*/, const TSet<UStateTreeState*>& /*MovedStates*/)
{
	UpdateCachedInfo();
}

}
