// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetCompilationHandler.h"
#include "UObject/WeakObjectPtr.h"

class UStateTreeState;
class UStateTree;

namespace UE::UAF::StateTree
{

class FStateTreeAssetCompilationHandler : public UE::UAF::Editor::FAssetCompilationHandler
{
public:
	explicit FStateTreeAssetCompilationHandler(UObject* InObject);

	// Non-constructor setup (e.g. binding SP delegates)
	void Initialize();

private:
	// IAssetCompilationHandler interface
	virtual void Compile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset) override;
	virtual UE::UAF::Editor::ECompileStatus GetCompileStatus(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const override;

	void UpdateCachedInfo();

	void HandleAssetChanged();
	void HandleStatesChanged(const TSet<UStateTreeState*>& /*AffectedStates*/, const FPropertyChangedEvent& /*PropertyChangedEvent*/);
	void HandleStateAdded(UStateTreeState* /*ParentState*/, UStateTreeState* /*NewState*/);
	void HandleStatesRemoved(const TSet<UStateTreeState*>& /*AffectedParents*/);
	void HandleStatesMoved(const TSet<UStateTreeState*>& /*AffectedParents*/, const TSet<UStateTreeState*>& /*MovedStates*/);

	TWeakObjectPtr<UStateTree> CachedStateTree;
};

}
