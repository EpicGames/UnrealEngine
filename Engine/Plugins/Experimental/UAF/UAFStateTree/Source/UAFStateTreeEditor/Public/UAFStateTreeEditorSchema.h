// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeEditorSchema.h"

#include "UAFStateTreeEditorSchema.generated.h"

UCLASS(MinimalAPI)
class UUAFStateTreeEditorSchema : public UStateTreeEditorSchema
{
	GENERATED_BODY()

	virtual EStateTreeTransitionEditingRules GetTransitionEditingRules(TNotNull<UStateTreeEditorData*> StateTree) const override
	{
		return EStateTreeTransitionEditingRules::AllowReactivation;
	}
};
