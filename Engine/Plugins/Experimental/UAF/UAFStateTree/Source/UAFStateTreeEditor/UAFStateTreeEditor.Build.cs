// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFStateTreeEditor : ModuleRules
	{
		public UAFStateTreeEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				    "UAFStateTree",
				    "MessageLog",
				    "EditorInteractiveToolsFramework",
				    "InteractiveToolsFramework",
				    "EditorFramework",
				    "UAF",
				    "UnrealEd",
				    "AssetDefinition",
					"UAFAnimGraph",
				    "RigVM",
				    "StructUtilsEditor",
				    "PropertyEditor",
					"ToolMenus",
					"UAFAnimNodeEditor",
					"UAFAnimNode"
			    });

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"WorkspaceEditor",
					"StateTreeEditorModule",
					"StateTreeModule",
					"PropertyBindingUtils",
					"UAFStateTreeUncookedOnly",
					"UAFEditor",
					"SlateCore",
					"EditorSubsystem",
					"Engine",
					"Slate",
					"RigVMDeveloper",
					"UAFUncookedOnly"
				}
			);
		}
	}
}