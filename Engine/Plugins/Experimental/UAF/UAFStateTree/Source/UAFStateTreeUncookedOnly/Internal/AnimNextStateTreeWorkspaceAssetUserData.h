// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"

#include "AnimNextAnimGraphWorkspaceAssetUserData.h"
#include "AnimNextStateTreeWorkspaceAssetUserData.generated.h"

UCLASS()
class UAnimNextStateTreeWorkspaceAssetUserData : public UAnimNextAnimGraphWorkspaceAssetUserData
{
	GENERATED_BODY()

protected:

	//~ Begin UAnimNextAnimGraphWorkspaceAssetUserData Interface
	virtual void GetRootAssetExport(FAssetRegistryTagsContext Context) const override;
	virtual void GetWorkspaceAssetExports(FAssetRegistryTagsContext Context) const override;
	//~ End UAnimNextAnimGraphWorkspaceAssetUserData Interface
};
