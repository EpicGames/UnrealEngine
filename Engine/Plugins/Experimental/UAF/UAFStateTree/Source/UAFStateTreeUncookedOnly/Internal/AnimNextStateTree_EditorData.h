// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/AnimNextAnimationGraph_EditorData.h"

#include "AnimNextStateTree_EditorData.generated.h"

UCLASS()
class UAFSTATETREEUNCOOKEDONLY_API UAnimNextStateTree_EditorData : public UUAFAnimGraph_EditorData
{
	GENERATED_BODY()
	
protected:
	virtual TSubclassOf<UAssetUserData> GetAssetUserDataClass() const override;

	friend class UAnimNextStateTreeFactory;
	friend class UAnimNextStateTreeTreeEditorData;

	// IRigVMClientHost interface
	virtual void RecompileVM() override;
	
	// UUAFRigVMAssetEditorData interface
	virtual TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> GetEntryClasses() const override;
	virtual void BuildFunctionHeadersContext(const FRigVMCompileSettings& InSettings, FAnimNextGetFunctionHeaderCompileContext& OutCompileContext) const override;
	virtual void OnPreCompileGetProgrammaticGraphs(const FRigVMCompileSettings& InSettings, FAnimNextGetGraphCompileContext& OutCompileContext) override;
	virtual void OnPostCompileVariables(const FRigVMCompileSettings& InSettings, const FAnimNextGetVariableCompileContext& InCompileContext) override;

private:
	// Property bag that forms the root parameters. Consists of all combined variables from shared variable assets/structs.
	UPROPERTY()
	FInstancedPropertyBag CombinedVariablesPropertyBag;
};