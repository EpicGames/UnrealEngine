// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeEditorData.h"

#include "UncookedOnlyUtils.h"
#include "AnimNextStateTree_EditorData.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditingSubsystem.h"

void UAnimNextStateTreeTreeEditorData::PostLoad()
{
	Super::PostLoad();
	if(const UAnimNextStateTree* OuterAnimNextStateTree = GetTypedOuter<UAnimNextStateTree>())
	{
		UAnimNextStateTree_EditorData* AnimNextEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UAnimNextStateTree_EditorData>(OuterAnimNextStateTree);	
		AnimNextEditorData->ModifiedDelegate.AddUObject(this, &UAnimNextStateTreeTreeEditorData::HandleStateTreeAssetChanges);
	}
}

void UAnimNextStateTreeTreeEditorData::CreateRootProperties(TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs)
{
	UStateTree* OuterStateTree = CastChecked<UStateTree>(GetOuter());
	UAnimNextStateTree* OuterAnimNextStateTree = CastChecked<UAnimNextStateTree>(OuterStateTree->GetOuter());

	UAnimNextStateTree_EditorData* AnimNextEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UAnimNextStateTree_EditorData>(OuterAnimNextStateTree);

	// Generate unique names for the incoming property descs to avoid changing the existing properties in the bag
	for (UE::PropertyBinding::FPropertyCreationDescriptor& CreationDesc : InOutCreationDescs)
	{
		int32 Index = CreationDesc.PropertyDesc.Name.GetNumber();
		while (OuterAnimNextStateTree->VariableDefaults.FindPropertyDescByName(CreationDesc.PropertyDesc.Name))
		{
			CreationDesc.PropertyDesc.Name = FName(CreationDesc.PropertyDesc.Name, Index++);
		}

		// Try and export default value from incoming property desc + data
		FString DefaultValue;
		if (CreationDesc.SourceProperty && CreationDesc.SourceContainerAddress)
		{
			void const* SourceAddress = CreationDesc.SourceProperty->ContainerPtrToValuePtr<void>(CreationDesc.SourceContainerAddress);
			CreationDesc.SourceProperty->ExportText_Direct(DefaultValue, SourceAddress, SourceAddress, nullptr, PPF_None);
		}
		
		const FAnimNextParamType ParamType(CreationDesc.PropertyDesc.ValueType, CreationDesc.PropertyDesc.ContainerTypes.GetFirstContainerType(), CreationDesc.PropertyDesc.ValueTypeObject);
		AnimNextEditorData->AddVariable(CreationDesc.PropertyDesc.Name, ParamType, *DefaultValue);
	}
}

void UAnimNextStateTreeTreeEditorData::HandleStateTreeAssetChanges(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType,
	UObject* InSubject) const
{	
	switch(InType)
	{
	case EAnimNextEditorDataNotifType::UndoRedo:
	case EAnimNextEditorDataNotifType::EntryAdded:
	case EAnimNextEditorDataNotifType::EntryRemoved:
	case EAnimNextEditorDataNotifType::EntryRenamed:
	case EAnimNextEditorDataNotifType::EntryAccessSpecifierChanged:
	case EAnimNextEditorDataNotifType::VariableTypeChanged:
	case EAnimNextEditorDataNotifType::VariableDefaultValueChanged:
	case EAnimNextEditorDataNotifType::VariableBindingChanged:
		{			
			UStateTree* StateTree = GetTypedOuter<UStateTree>();
			UE::StateTree::Delegates::OnParametersChanged.Broadcast(*StateTree);
			UStateTreeEditingSubsystem::MarkAsPubliclyModified(StateTree);
		}		
		break;
	default:
		break;
	}
}

const FInstancedPropertyBag& UAnimNextStateTreeTreeEditorData::GetRootParametersPropertyBag() const
{
	UStateTree* OuterStateTree = CastChecked<UStateTree>(GetOuter());
	UAnimNextStateTree* OuterAnimNextStateTree = CastChecked<UAnimNextStateTree>(OuterStateTree->GetOuter());
	UAnimNextStateTree_EditorData* AnimNextEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UAnimNextStateTree_EditorData>(OuterAnimNextStateTree);
	return AnimNextEditorData->CombinedVariablesPropertyBag;
}
