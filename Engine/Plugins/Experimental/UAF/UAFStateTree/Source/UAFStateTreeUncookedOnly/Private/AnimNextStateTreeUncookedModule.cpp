// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeUncookedModule.h"

IMPLEMENT_MODULE(UE::UAF::StateTree::FAnimNextStateTreeUncookedModule, UAFStateTreeUncookedOnly)