// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTreeWorkspaceAssetUserData.h"

#include "AnimNextStateTree.h"
#include "StateTreeEditorData.h"
#include "AnimNextStateTreeWorkspaceExports.h"
#include "Tasks/AnimNextStateTreeGraphInstanceTask.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "WorkspaceAssetRegistryInfo.h"

void UAnimNextStateTreeWorkspaceAssetUserData::GetRootAssetExport(FAssetRegistryTagsContext Context) const
{
	if (UAnimNextStateTree* AnimStateTree = CastChecked<UAnimNextStateTree>(GetOuter()))
	{
		const FName AnimStateTreeIdentifier = AnimStateTree->GetFName();
		UStateTree* StateTree = AnimStateTree->StateTree;
		FWorkspaceOutlinerItemExport& RootAssetExport = CachedExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(AnimStateTreeIdentifier, GetOuter()));
		RootAssetExport.GetData().InitializeAsScriptStruct(FAnimNextStateTreeOutlinerData::StaticStruct());
		RootAssetExport.GetData().GetMutable<FAnimNextStateTreeOutlinerData>().SoftAssetPtr = AnimStateTree;
	}
}

void UAnimNextStateTreeWorkspaceAssetUserData::GetWorkspaceAssetExports(FAssetRegistryTagsContext Context) const
{
	Super::GetWorkspaceAssetExports(Context);

	if (UAnimNextStateTree* AnimStateTree = CastChecked<UAnimNextStateTree>(GetOuter()))
	{		
		const FName AnimStateTreeIdentifier = AnimStateTree->GetFName();
		
		UStateTree* StateTree = AnimStateTree->StateTree;
		FWorkspaceOutlinerItemExport& RootAssetExport = CachedExports.Exports[0];

		if (const UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(StateTree->EditorData)) 
		{
			// Export each state as individual item as well (parented according to hierarchy)
			TMap<UStateTreeState*, FWorkspaceOutlinerItemExport> ParentExports;
			EditorData->VisitHierarchy([&CachedExports = CachedExports, RootAssetExport, &ParentExports, EditorData](UStateTreeState& State, const UStateTreeState* ParentState) -> EStateTreeVisitor
			{
				FWorkspaceOutlinerItemExport& StateExport = CachedExports.Exports.Add_GetRef(
				ParentState == nullptr ? FWorkspaceOutlinerItemExport(State.Name, RootAssetExport)
					: FWorkspaceOutlinerItemExport(State.Name, ParentExports.FindChecked(ParentState))
				);

				ParentExports.Add(&State, StateExport);

				StateExport.GetData().InitializeAsScriptStruct(FAnimNextStateTreeStateOutlinerData::StaticStruct());
				FAnimNextStateTreeStateOutlinerData& AssetData = StateExport.GetData().GetMutable<FAnimNextStateTreeStateOutlinerData>();
				AssetData.StateName = State.Name;
				AssetData.StateId = State.ID;
				AssetData.bIsLeafState = State.Children.IsEmpty(); 
				AssetData.Type = State.Type;
				AssetData.SelectionBehavior = State.SelectionBehavior;

				const FStateTreeEditorColor* StateTreeEditorColor = EditorData->FindColor(State.ColorRef);

				AssetData.Color = StateTreeEditorColor ? FSlateColor(StateTreeEditorColor->Color) : FSlateColor::UseForeground();

				// Retrieve and add AssetReferences for all tasks part of this state
				{
					TArray<const UObject*> ReferencedObjects;
					for (FStateTreeEditorNode& TaskNode : State.Tasks)
					{
						if(TaskNode.Node.IsValid() && TaskNode.Node.GetScriptStruct()->IsChildOf(FAnimNextStateTreeTaskBase::StaticStruct()))
						{
							if (TaskNode.Instance.IsValid())
							{							
								TaskNode.Node.Get<FAnimNextStateTreeTaskBase>().GetObjectReferences(ReferencedObjects, FStateTreeDataView(TaskNode.Instance.GetScriptStruct(), TaskNode.Instance.GetMutableMemory()));
							}
						}
					}

					for (const UObject* ReferredObject : ReferencedObjects)
					{
						// GetObjectReferences will return null for unset object reference(s)
						if (ReferredObject != nullptr)
						{
							FSoftObjectPath Path = ReferredObject;
							FWorkspaceOutlinerItemExport& GraphReference = CachedExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(FName(Path.ToString()), StateExport));
							GraphReference.GetData().InitializeAsScriptStruct(FWorkspaceOutlinerAssetReferenceItemData::StaticStruct());
							FWorkspaceOutlinerAssetReferenceItemData& ReferenceData = GraphReference.GetData().GetMutable<FWorkspaceOutlinerAssetReferenceItemData>();
							ReferenceData.ReferredObjectPath = Path;
						}
						
					}
				}
				
				
				return EStateTreeVisitor::Continue;
			});
		}
	}
}
