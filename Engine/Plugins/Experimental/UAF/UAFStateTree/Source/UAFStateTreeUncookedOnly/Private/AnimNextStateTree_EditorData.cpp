// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextStateTree_EditorData.h"

#include "AnimGraphUncookedOnlyUtils.h"
#include "UAFCompilationScope.h"
#include "AnimNextStateTree.h"
#include "AnimNextStateTreeEditorData.h"
#include "AnimNextStateTreeEditorOnlyTypes.h"
#include "AnimNextStateTreeWorkspaceAssetUserData.h"
#include "Conditions/AnimNextStateTreeRigVMConditionBase.h"
#include "Compilation/AnimNextGetGraphCompileContext.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "StateTree.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeEditingSubsystem.h"
#include "StateTreeEditorData.h"
#include "StateTreeState.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Entries/AnimNextAnimationGraphEntry.h"
#include "Graph/AnimNextAnimationGraphSchema.h"
#include "Graph/RigUnit_AnimNextGraphRoot.h"
#include "Graph/RigDecorator_AnimNextCppTrait.h"
#include "Graph/RigUnit_AnimNextTraitStack.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/RigVMGraph.h"
#include "Tasks/AnimNextStateTreeRigVMTaskBase.h"
#include "TraitCore/TraitRegistry.h"
#include "TraitCore/Trait.h"
#include "AnimStateTreeTrait.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Traits/BlendStackTrait.h"
#include "Traits/BlendSmoother.h"
#include "Traits/BlendSmootherPerBoneTraitData.h"

TSubclassOf<UAssetUserData> UAnimNextStateTree_EditorData::GetAssetUserDataClass() const
{
	return UAnimNextStateTreeWorkspaceAssetUserData::StaticClass();
}

void UAnimNextStateTree_EditorData::RecompileVM()
{
	UUAFAnimGraph_EditorData::RecompileVM();

	// Skip further state tree compilation if we had errors in regular asset compilation. It can create spurious errors
	if (bErrorsDuringCompilation)
	{
		return;
	}

	UAnimNextStateTree* AnimationStateTree = UE::UAF::UncookedOnly::FUtils::GetAsset<UAnimNextStateTree>(this);
	UE::UAF::UncookedOnly::FCompilationScope CompileScope(AnimationStateTree);

	UStateTree* InnerStateTree = AnimationStateTree->StateTree;

	// Recompile ST as we just updated our variables propertybag. It may have changed from an external shared variables update.
	FStateTreeCompilerLog Log;
	UStateTreeEditingSubsystem::CompileStateTree(InnerStateTree, Log);

	FCompilerResultsLog& CompileLog = UE::UAF::UncookedOnly::FCompilationScope::GetLogForObject(AnimationStateTree);
	TArray<TSharedRef<FTokenizedMessage>> Messages = Log.ToTokenizedMessages();
	for (const TSharedRef<FTokenizedMessage>& Message : Messages)
	{
		CompileLog.AddTokenizedMessage(Message);
	}
}

TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> UAnimNextStateTree_EditorData::GetEntryClasses() const
{
	static const TSubclassOf<UUAFRigVMAssetEntry> Classes[] =
{
		UAnimNextVariableEntry::StaticClass(),
		UAnimNextAnimationGraphEntry::StaticClass(),
		UUAFSharedVariablesEntry::StaticClass(),
	};

	return Classes;
}

void UAnimNextStateTree_EditorData::BuildFunctionHeadersContext(const FRigVMCompileSettings& InSettings, FAnimNextGetFunctionHeaderCompileContext& OutCompileContext) const
{
	Super::BuildFunctionHeadersContext(InSettings, OutCompileContext);

	if (UAnimNextStateTree* AnimStateTree = UE::UAF::UncookedOnly::FUtils::GetAsset<UAnimNextStateTree>(this))
	{
		// Give child nodes a chance to add compile-time only variables
		FAnimNextStateTreeProgrammaticFunctionHeaderParams ProgrammaticFunctionHeaderParams(this, InSettings, RigVMClient, OutCompileContext);

		// Populate UID info on our nodes
		auto MakeNodeVariables = [&](UStateTreeState& State, UStateTreeState* ParentState)
		{
			for (const FStateTreeEditorNode& Node : State.EnterConditions)
			{
				// Manually check for each child node type, UStruct's do not reflect interfaces so we can't use an interface here.
				if (const FAnimNextStateTreeRigVMConditionBase* Condition = Node.Node.GetPtr<FAnimNextStateTreeRigVMConditionBase>())
				{
					FStateTreeBindableStructDesc Desc;
					Desc.StatePath = State.GetPath() + TEXT("/EnterConditions");
					Desc.Struct = Condition->GetInstanceDataType();
					Desc.Name = Node.GetName();
					Desc.ID = Node.ID;
					Desc.DataSource = EStateTreeBindableStructSource::Condition;

					// Graph gen isn't const since we need to cache generated names in the node
					const_cast<FAnimNextStateTreeRigVMConditionBase*>(Condition)->GetProgrammaticFunctionHeaders(ProgrammaticFunctionHeaderParams, &State, Node.GetInstance(), Desc);
				}
			}

			for (const FStateTreeTransition& Transition : State.Transitions)
			{
				for (const FStateTreeEditorNode& Node : Transition.Conditions)
				{
					// Manually check for each child node type, UStruct's do not reflect interfaces so we can't use an interface here.
					if (const FAnimNextStateTreeRigVMConditionBase* Condition = Node.Node.GetPtr<FAnimNextStateTreeRigVMConditionBase>())
					{
						FStateTreeBindableStructDesc Desc;
						Desc.Struct = Condition->GetInstanceDataType();
						Desc.Name = Node.GetName();
						Desc.ID = Node.ID;
						Desc.DataSource = EStateTreeBindableStructSource::Transition;

						// Graph gen isn't const since we need to cache generated names in the node
						const_cast<FAnimNextStateTreeRigVMConditionBase*>(Condition)->GetProgrammaticFunctionHeaders(ProgrammaticFunctionHeaderParams, &State, Node.GetInstance(), Desc);
					}
				}
			}

			for (const FStateTreeEditorNode& Node : State.Tasks)
			{
				// Manually check for each child node type, UStruct's do not reflect interfaces so we can't use an interface here.
				if (const FAnimNextStateTreeRigVMTaskBase* Condition = Node.Node.GetPtr<FAnimNextStateTreeRigVMTaskBase>())
				{
					FStateTreeBindableStructDesc Desc;
					Desc.StatePath = State.GetPath() + TEXT("/Tasks");
					Desc.Struct = Condition->GetInstanceDataType();
					Desc.Name = Node.GetName();
					Desc.ID = Node.ID;
					Desc.DataSource = EStateTreeBindableStructSource::Task;

					// Graph gen isn't const since we need to cache generated names in the node
					const_cast<FAnimNextStateTreeRigVMTaskBase*>(Condition)->GetProgrammaticFunctionHeaders(ProgrammaticFunctionHeaderParams, &State, Node.GetInstance(), Desc);
				}
			}

			return EStateTreeVisitor::Continue;
		};

		if (AnimStateTree->StateTree)
		{
			if (UStateTreeEditorData* StateTreeEditorData = Cast<UStateTreeEditorData>(AnimStateTree->StateTree->EditorData))
			{
				StateTreeEditorData->VisitHierarchy(MakeNodeVariables);
			}
		}
	}
}

void UAnimNextStateTree_EditorData::OnPreCompileGetProgrammaticGraphs(const FRigVMCompileSettings& InSettings, FAnimNextGetGraphCompileContext& OutCompileContext)
{
	Super::OnPreCompileGetProgrammaticGraphs(InSettings, OutCompileContext);

	if (UAnimNextStateTree* AnimStateTree = UE::UAF::UncookedOnly::FUtils::GetAsset<UAnimNextStateTree>(this))
	{
		URigVMGraph* Graph = NewObject<URigVMGraph>(this, NAME_None, RF_Transient);
		Graph->SetSchemaClass(UAnimNextAnimationGraphSchema::StaticClass());

		UAnimNextController* Controller = CastChecked<UAnimNextController>(RigVMClient.GetOrCreateController(Graph));
		UE::UAF::UncookedOnly::FAnimGraphUtils::SetupAnimGraph(FRigUnit_AnimNextGraphRoot::DefaultEntryPoint, Controller, false);

		if (Controller->GetGraph()->GetNodes().Num() != 1)
		{
			InSettings.ReportError(TEXT("Expected singular FRigUnit_AnimNextGraphRoot node"));
			return;
		}
		
		URigVMNode* EntryNode = Controller->GetGraph()->GetNodes()[0];			
		URigVMPin* BeginExecutePin = EntryNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextGraphRoot, Result));

		if (BeginExecutePin == nullptr)
		{
			InSettings.ReportError(TEXT("Failed to retrieve Result pin from FRigUnit_AnimNextGraphRoot node"));
			return;
		}

		URigVMUnitNode* TraitStackNode = Controller->AddUnitNode(FRigUnit_AnimNextTraitStack::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(-800.0f, 0.0f), FString(), false);

		if (TraitStackNode == nullptr)
		{
			InSettings.ReportError(TEXT("Failed to spawn FRigUnit_AnimNextTraitStack node"));
			return;
		}

		const FName BlendStackTraitName = Controller->AddTraitByName(TraitStackNode->GetFName(), *UE::UAF::FTraitRegistry::Get().Find(UE::UAF::FBlendStackCoreTrait::TraitUID)->GetTraitName(), INDEX_NONE, TEXT(""), false);
		
		if (BlendStackTraitName == NAME_None)
		{
			InSettings.ReportError(TEXT("Failed to add BlendStack trait to node"));
			return;
		}
		
		const FName StateTreeTraitName = Controller->AddTraitByName(TraitStackNode->GetFName(), *UE::UAF::FTraitRegistry::Get().Find(UE::UAF::FStateTreeTrait::TraitUID)->GetTraitName(), INDEX_NONE, TEXT(""), false);

		if (StateTreeTraitName == NAME_None)
		{
			InSettings.ReportError(TEXT("Failed to add StateTree trait to node"));
			return;
		}

		const FName SmootherTraitName = Controller->AddTraitByName(TraitStackNode->GetFName(), *UE::UAF::FTraitRegistry::Get().Find(UE::UAF::FBlendSmootherCoreTrait::TraitUID)->GetTraitName(), INDEX_NONE, TEXT(""), false);

		if (SmootherTraitName == NAME_None)
		{
			InSettings.ReportError(TEXT("Failed to add Blend Smoother Core trait to node"));
			return;
		}
					
		const FName SmootherPerBoneTraitName = Controller->AddTraitStruct(TraitStackNode, TInstancedStruct<FAnimNextBlendSmootherPerBoneCoreTraitSharedData>::Make(), INDEX_NONE, false, false);

		if (SmootherPerBoneTraitName == NAME_None)
		{
			InSettings.ReportError(TEXT("Failed to add Blend Smoother Per Bone Core trait to node"));
			return;
		}

		URigVMPin* StateTreeReferencePin = TraitStackNode->FindTrait(StateTreeTraitName, GET_MEMBER_NAME_STRING_CHECKED(FAnimNextStateTreeTraitSharedData, StateTreeReference));
		if (StateTreeReferencePin == nullptr)
		{
			InSettings.ReportError(TEXT("Failed to retrieve StateTreeReference pin"));
			return;
		}
		
		FStateTreeReference Ref;
		Ref.SetStateTree(AnimStateTree->StateTree);

		FString PinValue;
		FStateTreeReference::StaticStruct()->ExportText(PinValue, &Ref, nullptr, nullptr, 0, nullptr);
		Controller->SetPinDefaultValue(StateTreeReferencePin->GetPinPath(), PinValue, true, false);
		
		URigVMPin* TraitResult = TraitStackNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_AnimNextTraitStack, Result));
		if (TraitResult == nullptr)
		{
			InSettings.ReportError(TEXT("Failed to retrieve Result pin"));
			return;
		}

		if (!Controller->AddLink(TraitResult, BeginExecutePin, false))
		{
			InSettings.ReportError(TEXT("Failed to link TraitStack and Graph Output pins"));
			return;
		}

		// @TODO: This has to be the last graph or ST will not execute. Figure out why, graph order should not be implicitly required
		OutCompileContext.GetMutableProgrammaticGraphs().Add(Graph);
	}
}

void UAnimNextStateTree_EditorData::OnPostCompileVariables(const FRigVMCompileSettings& InSettings, const FAnimNextGetVariableCompileContext& InCompileContext)
{
	Super::OnPostCompileVariables(InSettings, InCompileContext);

	CombinedVariablesPropertyBag = GenerateCombinedPropertyBag(InSettings, InCompileContext);
}
