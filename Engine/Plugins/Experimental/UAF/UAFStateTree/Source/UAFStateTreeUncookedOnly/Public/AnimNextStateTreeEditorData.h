// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextStateTree.h"
#include "StateTreeEditorData.h"
#include "UncookedOnlyUtils.h"

#include "AnimNextStateTreeEditorData.generated.h"

enum class EAnimNextEditorDataNotifType : uint8;

UCLASS()
class UAFSTATETREEUNCOOKEDONLY_API UAnimNextStateTreeTreeEditorData : public UStateTreeEditorData
{
	GENERATED_BODY()

public:
	//~ Begin UStateTreeEditorData overrides
	virtual void PostLoad() override;
	virtual const FInstancedPropertyBag& GetRootParametersPropertyBag() const override;
	virtual void CreateRootProperties(TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs) override;
	//~ End UStateTreeEditorData overrides

protected:
	void HandleStateTreeAssetChanges(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject) const;

	friend class UAnimNextStateTree_EditorData;
};