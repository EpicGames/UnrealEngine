// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Engine/AssetUserData.h"
#include "StateTreeTypes.h"
#include "Styling/SlateColor.h"

#include "AnimNextStateTreeWorkspaceExports.generated.h"

USTRUCT()
struct FAnimNextStateTreeOutlinerData : public FAnimNextRigVMAssetOutlinerData
{
	GENERATED_BODY()
	
	FAnimNextStateTreeOutlinerData() = default;
};

USTRUCT()
struct FAnimNextStateTreeStateOutlinerData : public FWorkspaceOutlinerItemData
{
	GENERATED_BODY()
	
	FAnimNextStateTreeStateOutlinerData() = default;

	UPROPERTY()
	FName StateName = NAME_None;

	UPROPERTY()
	FGuid StateId;

	UPROPERTY()
	bool bIsLeafState = true;

	UPROPERTY()
	EStateTreeStateSelectionBehavior SelectionBehavior = EStateTreeStateSelectionBehavior::None;

	UPROPERTY()
	EStateTreeStateType Type = EStateTreeStateType::State;

	UPROPERTY()
	FSlateColor Color;
};
