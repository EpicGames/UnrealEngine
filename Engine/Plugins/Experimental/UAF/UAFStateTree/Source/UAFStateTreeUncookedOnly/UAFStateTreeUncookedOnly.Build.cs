// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFStateTreeUncookedOnly : ModuleRules
	{
		public UAFStateTreeUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[] 
			{
				"StateTreeEditorModule",
				"PropertyBindingUtils",
				"RigVM",
				"RigVMDeveloper"
			});
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"StateTreeModule",
					"Engine",
					"WorkspaceEditor",
					"SlateCore",
					"UAF",
					"UAFUncookedOnly",
					"UAFStateTree",
					"UAFAnimGraph",
					"UAFAnimNode",
					"UAFAnimGraphUncookedOnly"
				}
			);
		}
	}
}