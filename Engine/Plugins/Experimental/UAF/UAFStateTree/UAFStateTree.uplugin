{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "UAF State Tree",
	"Description" : "StateTree integration for UAF.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "StateTree",
			"Enabled": true
		},
		{
			"Name": "Workspace",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": true
		},		
		{
			"Name": "UAFAnimGraph",
			"Enabled": true
		},
		{
			"Name": "UAFAnimNode",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "UAFStateTree",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "UAFStateTreeUncookedOnly",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFStateTreeEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}
